// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class Blutility : ModuleRules
{
	public Blutility(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePathModuleNames.Add("AssetTools");

        PublicDependencyModuleNames.AddRange(new string[] {
			"EditorSubsystem",
			"MainFrame"
		});

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"AutomationController",
				"Core",
				"CoreUObject",
				"Engine",
                "InputCore",
				"Slate",
				"SlateCore",
				"EditorFramework",
				"UnrealEd",
				"Kismet",
				"AssetDefinition",
				"AssetRegistry",
				"AssetTools",
				"WorkspaceMenuStructure",
				"ContentBrowser",
				"ContentBrowserData",
				"ClassViewer",
				"CollectionManager",
                "PropertyEditor",
                "BlueprintGraph",
                "Json",
                "JsonUtilities",
				"UMG",
                "UMGEditor",
                "KismetCompiler",
				"ToolMenus",
				"ToolWidgets",
				"RHI",
				"RenderCore",
				"ImageWrapper",
				"ImageWriteQueue",
				"DeveloperSettings",
				"EditorToolEvents"
			}
			);
	}
}
