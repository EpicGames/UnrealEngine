// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorUtilityObject.h"
#include "IEditorUtilityExtension.h"
#include "ActorActionUtility.generated.h"

#define UE_API BLUTILITY_API

/** 
 * Base class for all actor action-related utilities
 * Any functions/events that are exposed on derived classes that have the correct signature will be
 * included as menu options when right-clicking on a group of actors in the level editor.
 */
UCLASS(MinimalAPI, Abstract, hideCategories=(Object), Blueprintable)
class UActorActionUtility : public UEditorUtilityObject, public IEditorUtilityExtension
{
	GENERATED_BODY()

public:
	/** Return the class that this actor action supports. Leave this blank to support all actor classes. */
	UE_DEPRECATED(5.2, "GetSupportedClasses() instead, but ideally you're not requesting this directly and are instead using the FAssetActionUtilityPrototype to wrap access to an unload utility asset.")
	UFUNCTION(BlueprintPure, BlueprintImplementableEvent, Category="Assets", meta=(DeprecatedFunction, DeprecationMessage="If you were just returning a single class add it to the SupportedClasses array (you can find it listed in the Class Defaults).  If you were doing complex logic to simulate having multiple classes act as filters, add them to the SupportedClasses array.  If you were doing 'other' logic, you'll need to do that upon action execution."))
	UE_API UClass* GetSupportedClass() const;
	
	/**
     * Gets the statically determined supported classes, these classes are used as a first pass filter when determining
     * if we can utilize this asset utility action on the asset.
     */
    UFUNCTION(BlueprintPure, Category = "Assets")
    const TArray<TSoftClassPtr<UObject>>& GetSupportedClasses() const { return SupportedClasses; }

public:
    // Begin UObject
	UE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	// End UObject
	
private:
	/**
	 * For simple Asset Action's you should fill out the supported class here.
	 */
	UPROPERTY(EditDefaultsOnly, Category="Assets", meta=(AllowAbstract))
	TArray<TSoftClassPtr<UObject>> SupportedClasses;
};

#undef UE_API
