// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "EditorUtilityObject.h"
#include "UObject/ScriptMacros.h"
#include "IEditorUtilityExtension.h"
#include "AssetActionUtility.generated.h"

#define UE_API BLUTILITY_API

namespace AssetActionUtilityTags
{
	extern const FName BlutilityTagVersion;
	extern const FName SupportedClasses;
	extern const FName IsActionForBlueprint;
	extern const FName CallableFunctions;
	extern const FName SupportedConditions;
}

struct FAssetData;

USTRUCT()
struct FAssetActionSupportCondition
{
	GENERATED_BODY()

	/**
	 * This is a content browser styled filter.  For example, ..._N AND VirtualTextureStreaming=FALSE, would require that
	 * asset tag VirtualTextureStreaming be false, and the asset name end in _N.
	 *
	 * You can learn more about the content browser search syntax in the "Advanced Search Syntax" documentation.
	 */
	UPROPERTY(EditAnywhere, Category=Condition)
	FString Filter;

	/**
	 * This is the failure reason to reply to the user with if the condition above fails.
	 * If you fill in the reason, it will override the default failure text in the tooltip for the function menu option.
	 */
	UPROPERTY(EditAnywhere, Category=Condition, meta=(MultiLine=true))
	FString FailureReason;

	/**
	 * If this filter does not pass, show the corresponding functions from the menu anyways.
	 * If true, the menu option will display with the error tooltip, but be disabled.
	 * If false, the menu options will be removed entirely.
	 */
	UPROPERTY(EditDefaultsOnly, Category = "Asset Support")
	bool bShowInMenuIfFilterFails = true;
};

/** 
 * Base class for all asset action-related utilities
 * Any functions/events that are exposed on derived classes that have the correct signature will be
 * included as menu options when right-clicking on a group of assets in the content browser.
 */
UCLASS(MinimalAPI, Abstract, hideCategories=(Object), Blueprintable)
class UAssetActionUtility : public UEditorUtilityObject, public IEditorUtilityExtension
{
	GENERATED_BODY()

public:
	/**
	 * Return the class that this asset action supports (if not implemented, it will show up for all asset types)
	 * Do not do custom logic here based on the currently selected assets.
	 */
	UE_DEPRECATED(5.2, "GetSupportedClasses() instead, but ideally you're not requesting this directly and are instead using the FAssetActionUtilityPrototype to wrap access to an unload utility asset.")
	UFUNCTION(BlueprintPure, BlueprintImplementableEvent, Category="Assets", meta=(DeprecatedFunction, DeprecationMessage="If you were just returning a single class add it to the SupportedClasses array (you can find it listed in the Class Defaults).  If you were doing complex logic to simulate having multiple classes act as filters, add them to the SupportedClasses array.  If you were doing 'other' logic, you'll need to do that upon action execution."))
	UE_API UClass* GetSupportedClass() const;

	/**
	 * Returns whether or not this action is designed to work specifically on Blueprints (true) or on all assets (false).
	 * If true, GetSupportedClass() is treated as a filter against the Parent Class of selected Blueprint assets.
	 * @note Returns the value of bIsActionForBlueprints by default.
	 */
	UFUNCTION(BlueprintPure, BlueprintNativeEvent, Category="Assets")
	UE_API bool IsActionForBlueprints() const;

	/**
	 * Gets the statically determined supported classes, these classes are used as a first pass filter when determining
	 * if we can utilize this asset utility action on the asset.
	 */
	UFUNCTION(BlueprintPure, Category = "Assets")
	const TArray<TSoftClassPtr<UObject>>& GetSupportedClasses() const { return SupportedClasses; }

public:
	// Begin UObject
	UE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	// End UObject

protected:
	/**
	 * Is this action designed to work specifically on Blueprints (true) or on all assets (false).
	 * If true, SupportedClasses is treated as a filter against the Parent Class of selected Blueprint assets.
	 */
	UPROPERTY(EditDefaultsOnly, Category="Asset Support")
	bool bIsActionForBlueprints = false;

	/**
	 * The supported classes controls the list of classes that may be operated on by all of the asset functions in this
	 * utility class.
	 * @note When bIsActionForBlueprints is true, this will compare against the generated class of any Blueprint assets.
	 */
	UPROPERTY(EditDefaultsOnly, Category="Asset Support", meta=(AllowAbstract, ShowObjectRootClass))
	TArray<TSoftClassPtr<UObject>> SupportedClasses;

	/**
	 * The supported conditions for any asset to use these utility functions.  Note: all of these conditions must pass
	 * in sequence.  So if you have earlier failure conditions you want to be run first, put them first in the list,
	 * if those fail, their failure reason will be handled first.
	 */
	UPROPERTY(EditDefaultsOnly, Category="Asset Support")
	TArray<FAssetActionSupportCondition> SupportedConditions;
};

#undef UE_API
