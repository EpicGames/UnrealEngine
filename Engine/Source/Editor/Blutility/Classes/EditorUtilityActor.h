// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Base class of any editor-only actors
 */

#pragma once

#include "Components/InputComponent.h"
#include "CoreMinimal.h"
#include "GameFramework/Actor.h"
#include "Math/Transform.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/ScriptMacros.h"
#include "UObject/UObjectGlobals.h"

#include "EditorUtilityActor.generated.h"

#define UE_API BLUTILITY_API

class UInputComponent;
class UObject;
struct FFrame;
struct FPropertyChangedEvent;

UCLASS(MinimalAPI, Abstract, Blueprintable, meta = (ShowWorldContextPin))
class AEditorUtilityActor : public AActor
{
	GENERATED_UCLASS_BODY()

	// Standard function to execute
	UFUNCTION(BlueprintCallable, BlueprintImplementableEvent, Category = "Editor")
	UE_API void Run();

	UE_API virtual void OnConstruction(const FTransform& Transform) override;
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	/** Returns the current InputComponent on this utility actor. This will be NULL unless bReceivesEditorInput is set to true. */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "Input|Editor")
	UInputComponent* GetInputComponent() const
	{ 
		return EditorOnlyInputComponent.Get();
	}
	
	UFUNCTION(BlueprintSetter, Category = "Input|Editor")
	UE_API void SetReceivesEditorInput(bool bInValue);
	
	UFUNCTION(BlueprintGetter, BlueprintPure, Category = "Input|Editor")
	bool GetReceivesEditorInput() const
	{
		return bReceivesEditorInput;
	}	
	
private:

	/** Creates the EditorOnlyInputComponent if it does not already exist and registers all subobject callbacks to it */
	UE_API void CreateEditorInput();

	/** Removes the EditorOnlyInputComponent from this utility actor */
	UE_API void RemoveEditorInput();
	
	UPROPERTY(Transient, DuplicateTransient)
	TObjectPtr<UInputComponent> EditorOnlyInputComponent;
	
	/** If set to true, then this actor will be able to recieve input delegate callbacks when in the editor. */
	UPROPERTY(EditAnywhere, Category = "Input|Editor", BlueprintSetter = SetReceivesEditorInput, BlueprintGetter = GetReceivesEditorInput)
	bool bReceivesEditorInput = false;
};

#undef UE_API
