// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Base class of any editor-only actor components
 */

#pragma once

#include "Components/ActorComponent.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "EditorUtilityActorComponent.generated.h"

class UObject;


UCLASS(MinimalAPI, Abstract, Blueprintable, meta = (ShowWorldContextPin))
class UEditorUtilityActorComponent : public UActorComponent
{
	GENERATED_UCLASS_BODY()

	
};
