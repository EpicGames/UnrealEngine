// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Blueprint for editor utilities
 */

#pragma once

#include "Engine/Blueprint.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "EditorUtilityBlueprint.generated.h"

#define UE_API BLUTILITY_API

class UObject;

UCLASS(MinimalAPI)
class UEditorUtilityBlueprint : public UBlueprint
{
	GENERATED_UCLASS_BODY()

	// UBlueprint interface
	UE_API virtual bool SupportedByDefaultBlueprintFactory() const override;
	UE_API virtual bool AlwaysCompileOnLoad() const override;
	// End of UBlueprint interface
};

#undef UE_API
