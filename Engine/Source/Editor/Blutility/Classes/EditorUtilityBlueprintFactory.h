// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * Factory for utility blueprints
 */

#include "CoreMinimal.h"
#include "Factories/Factory.h"
#include "Templates/SharedPointer.h"
#include "Templates/SubclassOf.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "EditorUtilityBlueprintFactory.generated.h"

#define UE_API BLUTILITY_API

class FFeedbackContext;
class SWindow;
class UClass;
class UObject;

UCLASS(MinimalAPI)
class UEditorUtilityBlueprintFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	// The parent class of the created blueprint
	UPROPERTY(EditAnywhere, Category=BlueprintFactory, meta=(AllowAbstract = "", BlueprintBaseOnly = ""))
	TSubclassOf<class UObject> ParentClass;

	// UFactory interface
	UE_API virtual bool ConfigureProperties() override;
	UE_API virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	UE_API virtual bool CanCreateNew() const override;
	// End of UFactory interface

	/** Handler for when a class is picked in the class picker */
	UE_API void OnClassPicked(UClass* InChosenClass);

protected:
	/** A pointer to the window that is asking the user to select a parent class */
	TWeakPtr<SWindow> PickerWindow;
};

#undef UE_API
