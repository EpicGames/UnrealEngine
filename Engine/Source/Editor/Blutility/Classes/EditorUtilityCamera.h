// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Base class of any editor-only camera actors. Should be used when creating editor-only capture tools, etc.
 */

#pragma once

#include "Camera/CameraActor.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "EditorUtilityCamera.generated.h"

class UObject;


UCLASS(MinimalAPI, Abstract, Blueprintable, meta = (ShowWorldContextPin))
class AEditorUtilityCamera : public ACameraActor
{
	GENERATED_UCLASS_BODY()


};
