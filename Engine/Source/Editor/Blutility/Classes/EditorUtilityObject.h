// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Base class of any editor-only objects
 */

#pragma once

#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "IAssetRegistryTagProviderInterface.h"

#include "EditorUtilityObject.generated.h"

#define UE_API BLUTILITY_API


UCLASS(MinimalAPI, Abstract, Blueprintable, meta = (ShowWorldContextPin))
class UEditorUtilityObject : public UObject, public IAssetRegistryTagProviderInterface
{
	GENERATED_UCLASS_BODY()

	// Standard function to execute
	UFUNCTION(BlueprintCallable, BlueprintImplementableEvent, Category = "Editor")
	UE_API void Run();

	// we use this to call tool started delegate
	UE_API virtual void PostLoad() override;

	//~ Begin IAssetRegistryTagProviderInterface interface
	virtual bool ShouldAddCDOTagsToBlueprintClass() const override 
	{ 
		return true; 
	}
	//~ End IAssetRegistryTagProviderInterface interface

protected:
	/** Run this editor utility on start-up (after asset discovery)? */
	UPROPERTY(Category=Settings, EditDefaultsOnly, AssetRegistrySearchable, DisplayName="Run on Start-up")
	bool bRunEditorUtilityOnStartup = false;
};

#undef UE_API
