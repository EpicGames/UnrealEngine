// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Engine/World.h"
#include "IAssetRegistryTagProviderInterface.h"
#include "Misc/AsyncTaskNotification.h"
#include "Templates/UniquePtr.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "EditorUtilityTask.generated.h"

#define UE_API BLUTILITY_API

class FAsyncTaskNotification;
class FText;
class UEditorUtilitySubsystem;
struct FFrame;

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnEditorUtilityTaskDynamicDelegate, UEditorUtilityTask*, Task);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnEditorUtilityTaskDelegate, UEditorUtilityTask* /*Task*/);

/**
 * 
 */
UCLASS(MinimalAPI, Abstract, Blueprintable, meta = (ShowWorldContextPin))
class UEditorUtilityTask : public UObject, public IAssetRegistryTagProviderInterface
{
	GENERATED_BODY()

public:
	FOnEditorUtilityTaskDelegate OnFinished;

public:
	UE_API UEditorUtilityTask();

	UFUNCTION()
	UE_API void Run();

	//~ Begin IAssetRegistryTagProviderInterface interface
	virtual bool ShouldAddCDOTagsToBlueprintClass() const override
	{
		return true;
	}
	//~ End IAssetRegistryTagProviderInterface interface

	UE_API virtual UWorld* GetWorld() const override;

public:
	UFUNCTION(BlueprintCallable, Category=Task)
	UE_API void FinishExecutingTask(const bool bSuccess = true);

	UFUNCTION(BlueprintCallable, Category = Task)
	UE_API void SetTaskNotificationText(const FText& Text);

	// Calls CancelRequested() and ReceiveCancelRequested()
	UE_API void RequestCancel();

	UFUNCTION(BlueprintCallable, Category = Task)
	UE_API bool WasCancelRequested() const;

protected:
	virtual void BeginExecution() {}
	virtual void CancelRequested() {}
	UE_API virtual FText GetTaskTitle() const;

	UFUNCTION(BlueprintImplementableEvent, Category=Task, meta=(DisplayName="BeginExecution"))
	UE_API void ReceiveBeginExecution();

	UFUNCTION(BlueprintImplementableEvent, Category=Task, meta=(DisplayName="CancelRequested"))
	UE_API void ReceiveCancelRequested();

	UFUNCTION(BlueprintImplementableEvent, Category=Task)
	UE_API FText GetTaskTitleOverride() const;

private:
	// Calls GetTaskTitle() and GetTaskTitleOverride()
	UE_API void CreateNotification();

	// Calls BeginExecution() and ReceiveBeginExecution()
	UE_API void StartExecutingTask();

protected:
	/** Run this editor utility on start-up (after asset discovery)? */
	UPROPERTY(Category=Settings, EditDefaultsOnly, AssetRegistrySearchable, DisplayName="Run on Start-up")
	bool bRunEditorUtilityOnStartup = false;

private:
	UPROPERTY(Transient)
	TObjectPtr<UEditorUtilitySubsystem> MyTaskManager;

	UPROPERTY(Transient)
	TObjectPtr<UEditorUtilityTask> MyParentTask;

	UPROPERTY(Transient)
	bool bCancelRequested = false;

	bool Cached_GIsRunningUnattendedScript = false;

	TUniquePtr<FAsyncTaskNotification> TaskNotification;

	friend UEditorUtilitySubsystem;
};

#undef UE_API
