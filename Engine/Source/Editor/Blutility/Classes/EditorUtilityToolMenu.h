// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IAssetRegistryTagProviderInterface.h"
#include "UObject/ScriptMacros.h"
#include "ToolMenuEntryScript.h"
#include "ToolMenuSection.h"
#include "EditorUtilityToolMenu.generated.h"


UCLASS(MinimalAPI, Blueprintable, abstract)
class UEditorUtilityToolMenuEntry : public UToolMenuEntryScript, public IAssetRegistryTagProviderInterface
{
	GENERATED_BODY()

	//~ Begin IAssetRegistryTagProviderInterface interface
	virtual bool ShouldAddCDOTagsToBlueprintClass() const override
	{
		return true;
	}
	//~ End IAssetRegistryTagProviderInterface interface

protected:
	/** Run this editor utility on start-up (after asset discovery)? */
	UPROPERTY(Category=Settings, EditDefaultsOnly, AssetRegistrySearchable, DisplayName="Run on Start-up")
	bool bRunEditorUtilityOnStartup = false;
};

UCLASS(MinimalAPI, Blueprintable, abstract)
class UEditorUtilityToolMenuSection : public UToolMenuSectionDynamic, public IAssetRegistryTagProviderInterface
{
	GENERATED_BODY()

	//~ Begin IAssetRegistryTagProviderInterface interface
	virtual bool ShouldAddCDOTagsToBlueprintClass() const override
	{
		return true;
	}
	//~ End IAssetRegistryTagProviderInterface interface

protected:
	/** Run this editor utility on start-up (after asset discovery)? */
	UPROPERTY(Category=Settings, EditDefaultsOnly, AssetRegistrySearchable, DisplayName="Run on Start-up")
	bool bRunEditorUtilityOnStartup = false;
};

