// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Widget for editor utilities
 */

#pragma once

#include "Blueprint/UserWidget.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "IAssetRegistryTagProviderInterface.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"
#include "UObject/UObjectGlobals.h"

#include "EditorUtilityWidget.generated.h"

#define UE_API BLUTILITY_API

class AActor;
class UEditorPerProjectUserSettings;
class UObject;

UCLASS(MinimalAPI, Abstract, meta = (ShowWorldContextPin), config = Editor)
class UEditorUtilityWidget : public UUserWidget, public IAssetRegistryTagProviderInterface
{
	GENERATED_BODY()

public:
	// The default action called when the widget is invoked if bAutoRunDefaultAction=true (it is never called otherwise)
	UFUNCTION(BlueprintCallable, BlueprintImplementableEvent, Category = "Editor")
	UE_API void Run();

	//~ Begin IAssetRegistryTagProviderInterface interface
	virtual bool ShouldAddCDOTagsToBlueprintClass() const override
	{
		return true;
	}
	//~ End IAssetRegistryTagProviderInterface interface

	UFUNCTION(BlueprintCallable, BlueprintPure = false, Category = "Editor Utility Widget")
	UE_API UWidget* FindChildWidgetByName(FName WidgetName) const;

	// Run the default action
	UE_API void ExecuteDefaultAction();

	bool ShouldAlwaysReregisterWithWindowsMenu() const
	{
		return bAlwaysReregisterWithWindowsMenu;
	}

	bool ShouldAutoRunDefaultAction() const
	{
		return bAutoRunDefaultAction;
	}

	/** Returns the default desired tab display name that was specified for this widget */
	FText GetTabDisplayName() const
	{
		return TabDisplayName;
	}

	virtual bool IsEditorUtility() const override { return true; }

protected:

	/** Used to send event OnEditorToolStarted event on initialize */
	UE_API virtual void NativeOnInitialized() override;

	/** The display name for tabs spawned with this widget */
	UPROPERTY(Category = Config, EditDefaultsOnly, BlueprintReadWrite, AssetRegistrySearchable)
	FText TabDisplayName;

	UPROPERTY(Category = Config, EditDefaultsOnly, BlueprintReadWrite, AssetRegistrySearchable)
	FString HelpText;

	// Should this widget always be re-added to the windows menu once it's opened
	UPROPERTY(Config, Category = Settings, EditDefaultsOnly)
	bool bAlwaysReregisterWithWindowsMenu;

	// Should this blueprint automatically run OnDefaultActionClicked, or should it open up a details panel to edit properties and/or offer multiple buttons
	UPROPERTY(Category = Settings, EditDefaultsOnly, BlueprintReadOnly)
	bool bAutoRunDefaultAction;

	/** Run this editor utility on start-up (after asset discovery)? */
	UPROPERTY(Category=Settings, EditDefaultsOnly, AssetRegistrySearchable, DisplayName="Run on Start-up")
	bool bRunEditorUtilityOnStartup = false;
};

#undef UE_API
