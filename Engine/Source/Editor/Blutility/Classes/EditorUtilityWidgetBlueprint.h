// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Widget for editor utilities
 */

#pragma once

#include "Containers/Set.h"
#include "CoreMinimal.h"
#include "EditorUtilityWidget.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "WidgetBlueprint.h"
#include "Widgets/Docking/SDockTab.h"

#include "EditorUtilityWidgetBlueprint.generated.h"

#define UE_API BLUTILITY_API

class FSpawnTabArgs;
class SDockTab;
class SWidget;
class UBlueprint;
class UClass;
class UEditorUtilityWidget;
class UObject;
class UWorld;

enum class EAssetEditorCloseReason : uint8;
enum class EMapChangeType : uint8;

UCLASS(MinimalAPI)
class UEditorUtilityWidgetBlueprint : public UWidgetBlueprint
{
	GENERATED_BODY()

public:
	UE_API virtual void BeginDestroy() override;

	UE_API TSharedRef<SDockTab> SpawnEditorUITab(const FSpawnTabArgs& SpawnTabArgs);

	/** Creates the slate widget from the UMG widget */
	UE_API TSharedRef<SWidget> CreateUtilityWidget();

	/** Recreate the tab's content on recompile */
	UE_API void RegenerateCreatedTab(UBlueprint* RecompiledBlueprint);
	
	UE_API void UpdateRespawnListIfNeeded(TSharedRef<SDockTab> TabBeingClosed);

	// UBlueprint interface
	UE_API virtual void GetReparentingRules(TSet< const UClass* >& AllowedChildrenOfClasses, TSet< const UClass* >& DisallowedChildrenOfClasses) const override;

	virtual bool AllowEditorWidget() const override { return true; }

	UEditorUtilityWidget* GetCreatedWidget() const
	{
		return CreatedUMGWidget;
	}

	void SetRegistrationName(FName InRegistrationName)
	{
		RegistrationName = InRegistrationName;
	}

	FName GetRegistrationName() const
	{
		return RegistrationName;
	}

	/** Returns the default desired tab display name that was specified for this widget */
	UE_API FText GetTabDisplayName() const;

	bool ShouldSpawnAsNomadTab() const
	{
		return bSpawnAsNomadTab;
	}

	UE_API virtual UWidgetEditingProjectSettings* GetRelevantSettings() override;
	UE_API virtual const UWidgetEditingProjectSettings* GetRelevantSettings() const override;

public:
	static UE_API void MarkTransientRecursive(UEditorUtilityWidget* UtilityWidget);

private:
	UE_API bool IsWidgetEnabled() const;

	UE_API void ChangeTabWorld(UWorld* World, EMapChangeType MapChangeType);

private:
	// Should the widget be enabled when running PIE
	UPROPERTY(Category = Settings, EditDefaultsOnly)
	bool bIsEnabledInPIE = false;

	// Should the widget be enabled when debugging BP
	UPROPERTY(Category = Settings, EditDefaultsOnly)
	bool bIsEnabledInDebugging = false;

	// Should the widget be spawned on a Nomad tab to be docked anywhere
	UPROPERTY(Category = Settings, EditDefaultsOnly)
	bool bSpawnAsNomadTab = false;

	FName RegistrationName;

	TWeakPtr<SDockTab> CreatedTab;

	UPROPERTY(Transient)
	TObjectPtr<UEditorUtilityWidget> CreatedUMGWidget;

	FDelegateHandle OnBlueprintCompileHandle;
};

#undef UE_API
