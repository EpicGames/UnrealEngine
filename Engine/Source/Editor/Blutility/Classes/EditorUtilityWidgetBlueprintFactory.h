// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * Factory for utility widget blueprints
 */

#include "Containers/EnumAsByte.h"
#include "CoreMinimal.h"
#include "Engine/Blueprint.h"
#include "Factories/Factory.h"
#include "Templates/SubclassOf.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "EditorUtilityWidgetBlueprintFactory.generated.h"

class FFeedbackContext;
class SWindow;
class UClass;
class UObject;

UCLASS(HideCategories = Object, MinimalAPI)
class UEditorUtilityWidgetBlueprintFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	// The type of blueprint that will be created
	UPROPERTY(EditAnywhere, Category = WidgetBlueprintFactory)
	TEnumAsByte<EBlueprintType> BlueprintType;

	// The parent class of the created blueprint
	UPROPERTY(EditAnywhere, Category = WidgetBlueprintFactory, meta = (AllowAbstract = ""))
	TSubclassOf<class UUserWidget> ParentClass;

	// UFactory interface
	virtual bool ConfigureProperties() override;
	virtual bool ShouldShowInNewMenu() const override { return true; }
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual bool CanCreateNew() const override;
	
	// End of UFactory interface

	UClass* RootWidgetClass;
};
