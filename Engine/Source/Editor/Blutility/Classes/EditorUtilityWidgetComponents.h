// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Editor Utility Specfic Widget Components
 * 
 * These exist to provide a UE5 style for Widget Blueprints. Historically
 * we conditionally changed styling in constructor to achive this style
 * however that causes issues with CDO comparision.
 */

#pragma once

#include "Components/Button.h"
#include "Components/CheckBox.h"
#include "Components/CircularThrobber.h"
#include "Components/ComboBoxKey.h"
#include "Components/ComboBoxString.h"
#include "Components/EditableText.h"
#include "Components/EditableTextBox.h"
#include "Components/ExpandableArea.h"
#include "Components/InputKeySelector.h"
#include "Components/ListView.h"
#include "Components/MultiLineEditableText.h"
#include "Components/MultiLineEditableTextBox.h"
#include "Components/ProgressBar.h"
#include "Components/ScrollBar.h"
#include "Components/ScrollBox.h"
#include "Components/Slider.h"
#include "Components/SpinBox.h"
#include "Components/Throbber.h"
#include "Components/TreeView.h"

#include "EditorUtilityWidgetComponents.generated.h"

#define UE_API BLUTILITY_API

UCLASS(MinimalAPI)
class UEditorUtilityButton : public UButton
{
	GENERATED_UCLASS_BODY()
};

UCLASS(MinimalAPI)
class UEditorUtilityCheckBox : public UCheckBox
{
	GENERATED_UCLASS_BODY()
};

UCLASS(MinimalAPI)
class UEditorUtilityCircularThrobber : public UCircularThrobber
{
	GENERATED_UCLASS_BODY()
};

UCLASS(MinimalAPI)
class UEditorUtilityComboBoxKey : public UComboBoxKey
{
	GENERATED_BODY()
	
public:
	UE_API UEditorUtilityComboBoxKey();
};

UCLASS(MinimalAPI)
class UEditorUtilityComboBoxString : public UComboBoxString
{
	GENERATED_UCLASS_BODY()
};

UCLASS(MinimalAPI)
class UEditorUtilityEditableText : public UEditableText
{
	GENERATED_UCLASS_BODY()
};

UCLASS(MinimalAPI)
class UEditorUtilityEditableTextBox : public UEditableTextBox
{
	GENERATED_UCLASS_BODY()
};

UCLASS(MinimalAPI)
class UEditorUtilityExpandableArea : public UExpandableArea
{
	GENERATED_UCLASS_BODY()
};

UCLASS(MinimalAPI)
class UEditorUtilityInputKeySelector : public UInputKeySelector
{
	GENERATED_UCLASS_BODY()
};

UCLASS(MinimalAPI)
class UEditorUtilityListView : public UListView
{
	GENERATED_UCLASS_BODY()
};

UCLASS(MinimalAPI)
class UEditorUtilityMultiLineEditableText : public UMultiLineEditableText
{
	GENERATED_UCLASS_BODY()
};

UCLASS(MinimalAPI)
class UEditorUtilityMultiLineEditableTextBox : public UMultiLineEditableTextBox
{
	GENERATED_UCLASS_BODY()
};

UCLASS(MinimalAPI)
class UEditorUtilityProgressBar : public UProgressBar
{
	GENERATED_UCLASS_BODY()
};

UCLASS(MinimalAPI)
class UEditorUtilityScrollBar : public UScrollBar
{
	GENERATED_UCLASS_BODY()
};

UCLASS(MinimalAPI)
class UEditorUtilityScrollBox : public UScrollBox
{
	GENERATED_UCLASS_BODY()
};

UCLASS(MinimalAPI)
class UEditorUtilitySlider : public USlider
{
	GENERATED_UCLASS_BODY()
};

UCLASS(MinimalAPI)
class UEditorUtilitySpinBox : public USpinBox
{
	GENERATED_UCLASS_BODY()
};

UCLASS(MinimalAPI)
class UEditorUtilityThrobber : public UThrobber
{
	GENERATED_UCLASS_BODY()
};

UCLASS(MinimalAPI)
class UEditorUtilityTreeView : public UTreeView
{
	GENERATED_UCLASS_BODY()
};

#undef UE_API
