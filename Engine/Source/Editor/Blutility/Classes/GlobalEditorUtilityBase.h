// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Base class of all global Blutility editor utilities.
 */

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Engine/World.h"
#include "Math/UnrealMathSSE.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"
#include "UObject/UObjectGlobals.h"

#include "GlobalEditorUtilityBase.generated.h"

#define UE_API BLUTILITY_API

class AActor;
class UEditorPerProjectUserSettings;
struct FFrame;

DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams( FForEachActorIteratorSignature, class AActor*, Actor, int32, Index );
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams( FForEachAssetIteratorSignature, class UObject*, Asset, int32, Index );


UCLASS(MinimalAPI, Abstract, hideCategories=(Object), Blueprintable, Deprecated)
class UDEPRECATED_GlobalEditorUtilityBase : public UObject
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(Category=Config, EditDefaultsOnly, BlueprintReadWrite, AssetRegistrySearchable)
	FString HelpText;

	UPROPERTY(Transient)
	bool bDirtiedSelectionSet;

	/** UObject interface */
	UE_API virtual UWorld* GetWorld() const override;

	UFUNCTION(BlueprintCallable, Category="Development|Editor")
	UE_API TArray<AActor*> GetSelectionSet();

	/**
	 * Attempts to find the actor specified by PathToActor in the current editor world
	 * @param	PathToActor	The path to the actor (e.g. PersistentLevel.PlayerStart)
	 * @return	A reference to the actor, or none if it wasn't found
	 */
	UFUNCTION(BlueprintPure, Category = "Development|Editor")
	UE_API AActor* GetActorReference(FString PathToActor);

	////////////////////////////

	// Should this blueprint automatically run OnDefaultActionClicked, or should it open up a details panel to edit properties and/or offer multiple buttons
	UPROPERTY(Category=Settings, EditDefaultsOnly, BlueprintReadOnly)
	bool bAutoRunDefaultAction;

	// The default action called when the blutility is invoked if bAutoRunDefaultAction=true (it is never called otherwise)
	UFUNCTION(BlueprintImplementableEvent)
	UE_API void OnDefaultActionClicked();

	////////////////////////////

	// Calls OnEachSelectedActor for each selected actor
	UFUNCTION(BlueprintCallable, Category="Development|Editor")
	UE_API void ForEachSelectedActor();

	// The method called for each selected actor when ForEachSelectedActor is called
	UPROPERTY(BlueprintAssignable)
	FForEachActorIteratorSignature OnEachSelectedActor;

	////////////////////////////

	// Calls OnEachSelectedAsset for each selected asset
	UFUNCTION(BlueprintCallable, Category="Development|Editor")
	UE_API void ForEachSelectedAsset();

	// The method called for each selected asset when ForEachSelectedAsset is called
	UPROPERTY(BlueprintAssignable)
	FForEachAssetIteratorSignature OnEachSelectedAsset;

	// Gets the set of currently selected assets
	UFUNCTION(BlueprintCallable, Category="Development|Editor")
	UE_API TArray<UObject*> GetSelectedAssets();

	///////////////////////////

	UFUNCTION(BlueprintPure, Category="Development|Editor")
	UE_API UEditorPerProjectUserSettings* GetEditorUserSettings();

	// Remove all actors from the selection set
	UFUNCTION(BlueprintCallable, Category="Development|Editor")
	UE_API void ClearActorSelectionSet();

	// Selects nothing in the editor (another way to clear the selection)
	UFUNCTION(BlueprintCallable, Category = "Development|Editor")
	UE_API void SelectNothing();

	// Set the selection state for the selected actor
	UFUNCTION(BlueprintCallable, Category="Development|Editor")
	UE_API void SetActorSelectionState(AActor* Actor, bool bShouldBeSelected);

	UFUNCTION(BlueprintCallable, Category="Development|Editor")
	UE_API void GetSelectionBounds(FVector& Origin, FVector& BoxExtent, float& SphereRadius);

	// Renames an asset (cannot move folders)
	UFUNCTION(BlueprintCallable, Category = "Development|Editor")
	UE_API void RenameAsset(UObject* Asset, const FString& NewName);

	////////////////////////////

	// Run the default action
	UE_API void ExecuteDefaultAction();

	// Handles notifying the editor if the recent command mucked with the selection set
	UE_API void PostExecutionCleanup();
};

#undef UE_API
