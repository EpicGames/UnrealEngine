// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Base class of all placed Blutility editor utilities.
 */

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Engine/EngineBaseTypes.h"
#include "GameFramework/Actor.h"
#include "Math/MathFwd.h"
#include "Math/Rotator.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "PlacedEditorUtilityBase.generated.h"

#define UE_API BLUTILITY_API

class UObject;
struct FFrame;

UCLASS(MinimalAPI, Abstract, hideCategories=(Object, Actor), Deprecated)
class ADEPRECATED_PlacedEditorUtilityBase : public AActor
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(Category=Config, EditDefaultsOnly, BlueprintReadWrite)
	FString HelpText;

	// AActor interface
	UE_API virtual void TickActor(float DeltaSeconds, ELevelTick TickType, FActorTickFunction& ThisTickFunction) override;
	virtual bool ShouldTickIfViewportsOnly() const override { return true; }
	// End of AActor interface

	/**
	 * Returns the current selection set in the editor.  Note that for non-editor builds, this will return an empty array
	 */
	UFUNCTION(BlueprintPure, Category = "Development|Editor")
	UE_API TArray<AActor*> GetSelectionSet();

	/**
	 * Gets information about the camera position for the primary level editor viewport.  In non-editor builds, these will be zeroed
	 *
	 * @param	CameraLocation	(out) Current location of the level editing viewport camera, or zero if none found
	 * @param	CameraRotation	(out) Current rotation of the level editing viewport camera, or zero if none found
	 * @return	Whether or not we were able to get a camera for a level editing viewport
	 */
	UFUNCTION(BlueprintPure, Category = "Development|Editor")
	UE_API bool GetLevelViewportCameraInfo(FVector& CameraLocation, FRotator& CameraRotation);

	/**
	* Sets information about the camera position for the primary level editor viewport.
	*
	* @param	CameraLocation	Location the camera will be moved to.
	* @param	CameraRotation	Rotation the camera will be set to.
	*/
	UFUNCTION(BlueprintCallable, Category = "Development|Editor")
	UE_API void SetLevelViewportCameraInfo(FVector CameraLocation, FRotator CameraRotation);

	// Remove all actors from the selection set
	UFUNCTION(BlueprintCallable, Category = "Development|Editor")
	UE_API void ClearActorSelectionSet();

	// Selects nothing in the editor (another way to clear the selection)
	UFUNCTION(BlueprintCallable, Category = "Development|Editor")
	UE_API void SelectNothing();

	// Set the selection state for the selected actor
	UFUNCTION(BlueprintCallable, Category = "Development|Editor")
	UE_API void SetActorSelectionState(AActor* Actor, bool bShouldBeSelected);

	/**
	* Attempts to find the actor specified by PathToActor in the current editor world
	* @param	PathToActor	The path to the actor (e.g. PersistentLevel.PlayerStart)
	* @return	A reference to the actor, or none if it wasn't found
	*/
	UFUNCTION(BlueprintPure, Category = "Development|Editor")
	UE_API AActor* GetActorReference(FString PathToActor);
};

#undef UE_API
