// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorActionUtility.h"

#include "EditorUtilityAssetPrototype.h"
#include "EditorUtilityBlueprint.h"
#include "UObject/AssetRegistryTagsContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActorActionUtility)

#define LOCTEXT_NAMESPACE "ActorActionUtility"

void UActorActionUtility::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);
	
	const bool IsUpToDate =
		!GetClass()->IsFunctionImplementedInScript("GetSupportedClass");

	if (IsUpToDate)
	{
		FAssetActionUtilityPrototype::AddTagsFor_Version(Context);
	}
	
	FAssetActionUtilityPrototype::AddTagsFor_SupportedClasses(SupportedClasses, Context);
	FAssetActionUtilityPrototype::AddTagsFor_CallableFunctions(this, Context);
}

#undef LOCTEXT_NAMESPACE
