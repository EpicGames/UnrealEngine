// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetActionUtility.h"

#include "EditorUtilityAssetPrototype.h"
#include "EditorUtilityBlueprint.h"
#include "AssetRegistry/AssetData.h"
#include "JsonObjectConverter.h"
#include "Misc/DataValidation.h"
#include "Serialization/JsonSerializerMacros.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/UObjectThreadContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetActionUtility)

#define LOCTEXT_NAMESPACE "AssetActionUtility"

bool UAssetActionUtility::IsActionForBlueprints_Implementation() const
{
	return bIsActionForBlueprints;
}

void UAssetActionUtility::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);
	
	const bool IsUpToDate =
		!GetClass()->IsFunctionImplementedInScript("GetSupportedClass");
	
	if (IsUpToDate)
	{
		FAssetActionUtilityPrototype::AddTagsFor_Version(Context);
	}
	
	FAssetActionUtilityPrototype::AddTagsFor_SupportedClasses(SupportedClasses, Context);
	FAssetActionUtilityPrototype::AddTagsFor_SupportedConditions(SupportedConditions, Context);
	if (!FUObjectThreadContext::Get().IsRoutingPostLoad)
	{
		FAssetActionUtilityPrototype::AddTagsFor_IsActionForBlueprints(IsActionForBlueprints(), Context);
	}
	FAssetActionUtilityPrototype::AddTagsFor_CallableFunctions(this, Context);
}
			
#undef LOCTEXT_NAMESPACE
