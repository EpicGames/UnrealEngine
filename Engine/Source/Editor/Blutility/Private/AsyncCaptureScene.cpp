// Copyright Epic Games, Inc. All Rights Reserved.

#include "AsyncCaptureScene.h"

#include "AssetCompilingManager.h"
#include "Camera/CameraComponent.h"
#include "Camera/CameraTypes.h"
#include "Components/SceneCaptureComponent2D.h"
#include "Containers/EnumAsByte.h"
#include "ContentStreaming.h"
#include "Engine/Engine.h"
#include "Engine/EngineTypes.h"
#include "Engine/GameEngine.h"
#include "Engine/SceneCapture2D.h"
#include "Engine/Texture.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/World.h"
#include "HAL/PlatformProperties.h"
#include "IAutomationControllerManager.h"
#include "IAutomationControllerModule.h"
#include "Materials/MaterialInterface.h"
#include "Math/Rotator.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "Templates/Casts.h"
#include "Templates/SharedPointer.h"

//----------------------------------------------------------------------//
// UAsyncCaptureScene
//----------------------------------------------------------------------//

#include UE_INLINE_GENERATED_CPP_BY_NAME(AsyncCaptureScene)

UAsyncCaptureScene::UAsyncCaptureScene()
{
}

UAsyncCaptureScene* UAsyncCaptureScene::CaptureSceneAsync(UCameraComponent* ViewCamera, TSubclassOf<ASceneCapture2D> SceneCaptureClass, int ResX, int ResY)
{
	UAsyncCaptureScene* AsyncTask = NewObject<UAsyncCaptureScene>();
	AsyncTask->Start(ViewCamera, SceneCaptureClass, ResX, ResY, 0);

	return AsyncTask;
}

UAsyncCaptureScene* UAsyncCaptureScene::CaptureSceneWithWarmupAsync(UCameraComponent* ViewCamera, TSubclassOf<ASceneCapture2D> SceneCaptureClass, int ResX, int ResY, int WarmUpFrames)
{
	UAsyncCaptureScene* AsyncTask = NewObject<UAsyncCaptureScene>();
	AsyncTask->Start(ViewCamera, SceneCaptureClass, ResX, ResY, WarmUpFrames);

	return AsyncTask;
}

void UAsyncCaptureScene::Start(UCameraComponent* ViewCamera, TSubclassOf<ASceneCapture2D> SceneCaptureClass, int ResX, int ResY, int InWarmUpFrames)
{
	const FVector CaptureLocation = ViewCamera->GetComponentLocation();
	const FRotator CaptureRotation = ViewCamera->GetComponentRotation();

	UWorld* World = ViewCamera->GetWorld();
	SceneCapture = World->SpawnActor<ASceneCapture2D>(SceneCaptureClass, CaptureLocation, CaptureRotation);
	if (SceneCapture)
	{
		USceneCaptureComponent2D* CaptureComponent = SceneCapture->GetCaptureComponent2D();

		if (CaptureComponent->TextureTarget == nullptr)
		{
			SceneCaptureRT = NewObject<UTextureRenderTarget2D>(this, TEXT("AsyncCaptureScene_RT"), RF_Transient);
			SceneCaptureRT->RenderTargetFormat = RTF_RGBA8_SRGB;
			SceneCaptureRT->InitAutoFormat(ResX, ResY);
			SceneCaptureRT->UpdateResourceImmediate(true);

			CaptureComponent->TextureTarget = SceneCaptureRT;
		}
		else
		{
			SceneCaptureRT = CaptureComponent->TextureTarget;
		}

		FMinimalViewInfo CaptureView;
		ViewCamera->GetCameraView(0, CaptureView);
		CaptureComponent->SetCameraView(CaptureView);
	}

	WarmUpFrames = FMath::Max(InWarmUpFrames, 1);
}

void UAsyncCaptureScene::Activate()
{
	if (!SceneCapture)
	{
		NotifyComplete(nullptr);
	}

	FinishLoadingBeforeScreenshot();

	USceneCaptureComponent2D* CaptureComponent = SceneCapture->GetCaptureComponent2D();
	CaptureComponent->CaptureScene();

	FinishLoadingBeforeScreenshot();

	for (int32 FrameCount = 0; FrameCount < WarmUpFrames; ++FrameCount)
	{
		CaptureComponent->CaptureScene();
	}

	NotifyComplete(SceneCaptureRT);
}

void UAsyncCaptureScene::NotifyComplete(UTextureRenderTarget2D* InTexture)
{
	Complete.Broadcast(InTexture);
	SetReadyToDestroy();

	if (SceneCapture)
	{
		SceneCapture->Destroy();
	}
}

void UAsyncCaptureScene::FinishLoadingBeforeScreenshot()
{
	FlushAsyncLoading();

	UWorld* CurrentWorld{ nullptr };
	// Make sure we finish all level streaming
	if (UGameEngine* GameEngine = Cast<UGameEngine>(GEngine))
	{
		if (UWorld* GameWorld = GameEngine->GetGameWorld())
		{
			CurrentWorld = GameWorld;
			GameWorld->FlushLevelStreaming(EFlushLevelStreamingType::Full);
		}
	}

	// Finish compiling the shaders if the platform doesn't require cooked data.
	if (!FPlatformProperties::RequiresCookedData())
	{
		UMaterialInterface::SubmitRemainingJobsForWorld(CurrentWorld);
		FAssetCompilingManager::Get().FinishAllCompilation();
		IAutomationControllerModule& AutomationController = IAutomationControllerModule::Get();
		AutomationController.GetAutomationController()->ResetAutomationTestTimeout(TEXT("shader compilation"));
	}

	// Force all mip maps to load before taking the screenshot.
	UTexture::ForceUpdateTextureStreaming();

	IStreamingManager::Get().StreamAllResources(0.0f);
}
