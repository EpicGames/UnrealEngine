// Copyright Epic Games, Inc. All Rights Reserved.

#include "AsyncImageExport.h"

#include "Async/Async.h"
#include "Async/TaskGraphInterfaces.h"
#include "Containers/ContainersFwd.h"
#include "Engine/Texture.h"
#include "Engine/TextureDefines.h"
#include "Engine/TextureRenderTarget2D.h"
#include "IImageWrapper.h"
#include "ImagePixelData.h"
#include "ImageWrapperHelper.h"
#include "ImageWriteQueue.h"
#include "ImageWriteTask.h"
#include "Math/Color.h"
#include "Math/IntPoint.h"
#include "Math/IntRect.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "RHI.h"
#include "RHICommandList.h"
#include "RenderingThread.h"
#include "Templates/Casts.h"
#include "Templates/Function.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"
#include "TextureResource.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

//----------------------------------------------------------------------//
// UAsyncImageExport
//----------------------------------------------------------------------//

#include UE_INLINE_GENERATED_CPP_BY_NAME(AsyncImageExport)

UAsyncImageExport::UAsyncImageExport()
{
}

UAsyncImageExport* UAsyncImageExport::ExportImageAsync(UTexture* Texture, const FString& OutputFile, int32 Quality)
{
	UAsyncImageExport* AsyncTask = NewObject<UAsyncImageExport>();
	AsyncTask->Quality = Quality;
	AsyncTask->Start(Texture, OutputFile);

	return AsyncTask;
}

void UAsyncImageExport::Start(UTexture* Texture, const FString& InOutputFile)
{
	TextureToExport = Texture;
	TargetFile = InOutputFile;
}

void UAsyncImageExport::Activate()
{
	TWeakObjectPtr<UAsyncImageExport> WeakThis(this);

	if (Cast<UTextureRenderTarget2D>(TextureToExport))
	{
		ENQUEUE_RENDER_COMMAND(ReadTextureCommand)(
			[this, WeakThis](FRHICommandListImmediate& RHICmdList)
		{
			FTextureResource* Resource = TextureToExport->GetResource();
			FRHITexture* ResourceRHI = Resource->GetTexture2DRHI();

			TArray<FColor> OutPixels;
			if (ensure(ResourceRHI))
			{
				OutPixels.SetNum(Resource->GetSizeX() * Resource->GetSizeY());
				RHICmdList.ReadSurfaceData(
					ResourceRHI,
					FIntRect(0, 0, Resource->GetSizeX(), Resource->GetSizeY()),
					OutPixels,
					FReadSurfaceDataFlags()
				);
			}

			FIntPoint ImageSize(Resource->GetSizeX(), Resource->GetSizeY());
			AsyncTask(ENamedThreads::GameThread, [this, WeakThis, ImageSize, Pixels = MoveTemp(OutPixels)]() mutable {
				if (WeakThis.IsValid())
				{
					ExportImage(MoveTemp(Pixels), ImageSize);
				}
			});
		});
	}
	else
	{
		FTextureSource& TextureSource = TextureToExport->Source;

		check( TextureSource.IsValid() );

		// todo : should use GetMipImage instead of GetMipData ; use FImage and FImageUtils for image export
		TArray64<uint8> TextureRawData;
		verify( TextureSource.GetMipData(TextureRawData, 0) );

		const int32 BytesPerPixel = static_cast<int32>(TextureSource.GetBytesPerPixel());
		const ETextureSourceFormat SourceFormat = TextureSource.GetFormat();

		const int32 Width = static_cast<int32>(TextureSource.GetSizeX());
		const int32 Height = static_cast<int32>(TextureSource.GetSizeY());

		TArray<FColor> OutPixels;
		OutPixels.SetNumZeroed(Width * Height);

		FColor Color(0, 0, 0, 0);
		for (int32 Y = 0; Y < Height; ++Y)
		{
			for (int32 X = 0; X < Width; ++X)
			{
				const int32 PixelByteOffset = (X + Y * Width) * BytesPerPixel;
				const uint8* PixelPtr = TextureRawData.GetData() + PixelByteOffset;

				switch (SourceFormat)
				{
					case TSF_BGRA8:
					case TSF_BGRE8:
					{
						Color = *((FColor*)PixelPtr);
						break;
					}
					case TSF_G8:
					{
						const uint8 Intensity = *PixelPtr;
						Color = FColor(Intensity, Intensity, Intensity, Intensity);
						break;
					}
					default:
					{
						ensureMsgf(false, TEXT("Unknown Format"));
						break;
					}
				}

				OutPixels[Y * Width + X] = Color;
			}
		}

		FIntPoint ImageSize(Width, Height);
		AsyncTask(ENamedThreads::GameThread, [this, WeakThis, ImageSize, Pixels = MoveTemp(OutPixels)]() mutable {
			if (WeakThis.IsValid())
			{
				ExportImage(MoveTemp(Pixels), ImageSize);
			}
		});
	}
}

void UAsyncImageExport::ExportImage(TArray<FColor>&& RawPixels, FIntPoint ImageSize)
{
	const FString AbsoluteFilePath = FPaths::IsRelative(TargetFile) ? FPaths::ConvertRelativePathToFull(TargetFile) : TargetFile;

	IImageWriteQueueModule* ImageWriteQueueModule = FModuleManager::Get().GetModulePtr<IImageWriteQueueModule>("ImageWriteQueue");

	FString Ext = FPaths::GetExtension(TargetFile);
	EImageFormat ImageFormat = ImageWrapperHelper::GetImageFormat(Ext);

	TUniquePtr<FImageWriteTask> ImageTask = MakeUnique<FImageWriteTask>();
	ImageTask->Format = ImageFormat;
	ImageTask->PixelData = MakeUnique<TImagePixelData<FColor>>(ImageSize, TArray64<FColor>(MoveTemp(RawPixels)));
	ImageTask->Filename = AbsoluteFilePath;
	ImageTask->bOverwriteFile = true;
	ImageTask->CompressionQuality = Quality;

	TWeakObjectPtr<UAsyncImageExport> WeakThis(this);
	ImageTask->OnCompleted = [this, WeakThis](bool bSuccess) {
		if (WeakThis.IsValid())
		{
			NotifyComplete(bSuccess);
		}
	};

	ImageWriteQueueModule->GetWriteQueue().Enqueue(MoveTemp(ImageTask));
}

void UAsyncImageExport::NotifyComplete(bool bSuccess)
{
	Complete.Broadcast(bSuccess);
	SetReadyToDestroy();
}
