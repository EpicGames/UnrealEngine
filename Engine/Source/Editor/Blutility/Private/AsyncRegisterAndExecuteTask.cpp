// Copyright Epic Games, Inc. All Rights Reserved.

#include "AsyncRegisterAndExecuteTask.h"

#include "Delegates/Delegate.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "EditorUtilitySubsystem.h"

//----------------------------------------------------------------------//
// UAsyncRegisterAndExecuteTask
//----------------------------------------------------------------------//

#include UE_INLINE_GENERATED_CPP_BY_NAME(AsyncRegisterAndExecuteTask)

UAsyncRegisterAndExecuteTask::UAsyncRegisterAndExecuteTask()
{
}

UAsyncRegisterAndExecuteTask* UAsyncRegisterAndExecuteTask::RegisterAndExecuteTask(UEditorUtilityTask* Task, UEditorUtilityTask* OptionalParentTask)
{
	UAsyncRegisterAndExecuteTask* AsyncTask = NewObject<UAsyncRegisterAndExecuteTask>();
	AsyncTask->Start(Task, OptionalParentTask);

	return AsyncTask;
}

void UAsyncRegisterAndExecuteTask::Start(UEditorUtilityTask* Task, UEditorUtilityTask* OptionalParentTask)
{
	Task->OnFinished.AddUObject(this, &UAsyncRegisterAndExecuteTask::HandleFinished);

	UEditorUtilitySubsystem* EditorUtilitySubsystem = GEditor->GetEditorSubsystem<UEditorUtilitySubsystem>();
	EditorUtilitySubsystem->RegisterAndExecuteTask(Task, OptionalParentTask);
}

void UAsyncRegisterAndExecuteTask::HandleFinished(UEditorUtilityTask* Task)
{
	OnFinished.Broadcast(Task);
}
