// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlutilityContentBrowserExtensions.h"

#include "Algo/AnyOf.h"
#include "Algo/Transform.h"
#include "AssetActionUtility.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "BlutilityMenuExtensions.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "ContentBrowserDelegates.h"
#include "ContentBrowserMenuContexts.h"
#include "ContentBrowserModule.h"
#include "Delegates/Delegate.h"
#include "EditorUtilityAssetPrototype.h"
#include "EditorUtilityBlueprint.h"
#include "EditorUtilityWidgetProjectSettings.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "IAssetTools.h"
#include "Logging/MessageLog.h"
#include "Misc/NamePermissionList.h"
#include "Modules/ModuleManager.h"
#include "Templates/Casts.h"
#include "Templates/SharedPointer.h"
#include "Templates/SubclassOf.h"
#include "Templates/UnrealTemplate.h"
#include "ToolMenus.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/UObjectIterator.h"

#define LOCTEXT_NAMESPACE "BlutilityContentBrowserExtensions"

namespace UE::Editor::Utilities
{
	bool bEnableExperimentalScriptedActionsClassSupport = false;
	FAutoConsoleVariableRef GCvarExpiermentalScriptedActionsClassSupport(
		TEXT("Editor.AssetActions.ExperimentalClassSupport"),
		bEnableExperimentalScriptedActionsClassSupport,
		TEXT("Enables experimental support for iterating the class hierarchy of blueprints for scripted asset actions."),
		FConsoleVariableDelegate());
}

void FBlutilityContentBrowserExtensions::InstallHooks()
{
	UToolMenus::RegisterStartupCallback(
		FSimpleMulticastDelegate::FDelegate::CreateStatic(&FBlutilityContentBrowserExtensions::RegisterMenus));
}

void FBlutilityContentBrowserExtensions::RegisterMenus()
{
	// Mark us as the owner of everything we add.
	FToolMenuOwnerScoped OwnerScoped("FBlutilityContentBrowserExtensions");

	UToolMenu* const Menu = UToolMenus::Get()->ExtendMenu("ContentBrowser.AssetContextMenu");
	if (!Menu)
	{
		return;
	}

	FToolMenuSection& Section = Menu->FindOrAddSection("CommonAssetActions");
	Section.AddDynamicEntry("BlutilityContentBrowserExtensions", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
	{
		UContentBrowserAssetContextMenuContext* const Context = InSection.FindContext<UContentBrowserAssetContextMenuContext>();
		if (!Context)
		{
			return;
		}

		const TArray<FAssetData>& SelectedAssets = Context->SelectedAssets;

		// Run thru the assets to determine if any meet our criteria
		TMap<TSharedRef<FAssetActionUtilityPrototype>, TSet<int32>> UtilityAndSelectionIndices;
		TArray<FAssetData> SupportedAssets;
		TMap<FAssetData, int32> ProcessedAssetIndices;
		
		if (SelectedAssets.Num() > 0)
		{
			FMessageLog EditorErrors("EditorErrors");
			
			// Resolve classes once to avoid doing it for each blutility
			TArray<UClass*> SelectedAssetClasses;
			Algo::Transform(SelectedAssets, SelectedAssetClasses, [](const FAssetData& Asset) { return Asset.GetClass(EResolveClass::Yes); });
			auto ProcessAssetAction = [&EditorErrors, &SupportedAssets, &ProcessedAssetIndices, &UtilityAndSelectionIndices, &SelectedAssets, &SelectedAssetClasses](const TSharedRef<FAssetActionUtilityPrototype>& ActionUtilityPrototype)
			{
				if (ActionUtilityPrototype->IsLatestVersion())
				{
					TArray<TSoftClassPtr<UObject>> SupportedClassPtrs = ActionUtilityPrototype->GetSupportedClasses();
					if (SupportedClassPtrs.Num() == 0)
					{
						return;
					}

					const bool bIsActionForBlueprints = ActionUtilityPrototype->AreSupportedClassesForBlueprints();

					for (int32 SelectedAssetIndex = 0; SelectedAssetIndex < SelectedAssets.Num(); ++SelectedAssetIndex)
					{
						const FAssetData& Asset = SelectedAssets[SelectedAssetIndex];
						bool bPassesClassFilter = false;
						if (bIsActionForBlueprints)
						{
							UClass* AssetClass = SelectedAssetClasses[SelectedAssetIndex];
							if (AssetClass && (AssetClass->IsChildOf<UBlueprint>() || AssetClass->IsChildOf<UBlueprintGeneratedClass>()))
							{
								if (UE::Editor::Utilities::bEnableExperimentalScriptedActionsClassSupport)
								{
									FString ClassPathAsString;
									if (AssetClass->IsChildOf<UBlueprintGeneratedClass>())
									{
										ClassPathAsString = Asset.GetObjectPathString();
									}
									else
									{
										Asset.GetTagValue(FBlueprintTags::GeneratedClassPath, ClassPathAsString);
									}
									FTopLevelAssetPath AssetClassPath(ClassPathAsString);

									const IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();
									TArray<FTopLevelAssetPath> AssetClassPathHierarchy;
									AssetRegistry.GetAncestorClassNames(AssetClassPath, AssetClassPathHierarchy);
									AssetClassPathHierarchy.Insert(AssetClassPath, 0);

									for (const FTopLevelAssetPath& ClassPath : AssetClassPathHierarchy)
									{
										bPassesClassFilter = Algo::AnyOf(SupportedClassPtrs,
											[ClassPath](const TSoftClassPtr<UObject>& ClassPtr)
											{
												return ClassPtr.ToSoftObjectPath() == FSoftObjectPath(ClassPath);
											});

										if (bPassesClassFilter)
										{
											break;
										}
									}
								}
								else
								{
									if (const UClass* Blueprint_ParentClass = UBlueprint::GetBlueprintParentClassFromAssetTags(Asset))
									{
										bPassesClassFilter = 
											Algo::AnyOf(SupportedClassPtrs, [Blueprint_ParentClass](const TSoftClassPtr<UObject>& ClassPtr){ return Blueprint_ParentClass->IsChildOf(ClassPtr.Get()); });
									}
								}
							}
						}
						else
						{
							UClass* AssetClass = SelectedAssetClasses[SelectedAssetIndex];
							// Is the asset the right kind?
							bPassesClassFilter = AssetClass != nullptr &&
								Algo::AnyOf(SupportedClassPtrs, [AssetClass](const TSoftClassPtr<UObject>& ClassPtr){ return AssetClass->IsChildOf(ClassPtr.Get()); });
						}

						if (bPassesClassFilter)
						{
							int32 Index = ProcessedAssetIndices.FindRef(Asset, INDEX_NONE);
							if (Index == INDEX_NONE)
							{
								Index = SupportedAssets.Add(Asset);
								ProcessedAssetIndices.Add(Asset, Index);
							}
							
							UtilityAndSelectionIndices.FindOrAdd(ActionUtilityPrototype).Add(Index);
						}
					}
				}
				else
				{
					const FAssetData& BlutilityAssetData =  ActionUtilityPrototype->GetUtilityBlueprintAsset();
					if (IAssetTools::Get().GetAssetClassPathPermissionList(EAssetClassAction::ViewAsset)->PassesFilter(BlutilityAssetData.AssetClassPath.ToString()))
					{
						EditorErrors.NewPage(LOCTEXT("ScriptedActions", "Scripted Actions"));
						TSharedRef<FTokenizedMessage> ErrorMessage = EditorErrors.Error();
						ErrorMessage->AddToken(FAssetNameToken::Create(BlutilityAssetData.GetObjectPathString(), FText::FromString(BlutilityAssetData.GetObjectPathString())));
						ErrorMessage->AddToken(FTextToken::Create(LOCTEXT("NeedsToBeUpdated", "needs to be re-saved and possibly upgraded.")));
					}
				}
			};

			// Check blueprint utils (we need to load them to query their validity against these assets)
			TArray<FAssetData> UtilAssets;
			FBlutilityMenuExtensions::GetBlutilityClasses(UtilAssets, UAssetActionUtility::StaticClass()->GetClassPathName());

			// Process asset based utilities
			for (const FAssetData& UtilAsset : UtilAssets)
			{
				if (const UClass* ParentClass = UBlueprint::GetBlueprintParentClassFromAssetTags(UtilAsset))
				{
					// We only care about UEditorUtilityBlueprint's that are compiling subclasses of UAssetActionUtility
					if (ParentClass->IsChildOf(UAssetActionUtility::StaticClass()))
					{
						ProcessAssetAction(MakeShared<FAssetActionUtilityPrototype>(UtilAsset));
					}
				}
			}

			// Don't warn errors if searching generated classes, since not all utilities may be updated to work with generated classes yet (Must be done piecemeal).
			const UEditorUtilityWidgetProjectSettings* EditorUtilitySettings = GetDefault<UEditorUtilityWidgetProjectSettings>();
			if (!EditorUtilitySettings->bSearchGeneratedClassesForScriptedActions)
			{
				EditorErrors.Notify(LOCTEXT("SomeProblemsWithAssetActionUtility", "There were some problems with some AssetActionUtility Blueprints."));
			}
		}

		FBlutilityMenuExtensions::CreateAssetBlutilityActionsMenu(InSection, MoveTemp(UtilityAndSelectionIndices), MoveTemp(SupportedAssets));
	}));
}

void FBlutilityContentBrowserExtensions::RemoveHooks()
{
	// Remove our startup delegate in case it's still around.
	UToolMenus::UnRegisterStartupCallback("FBlutilityContentBrowserExtensions");
	// Remove everything we added to UToolMenus.
	UToolMenus::UnregisterOwner("FBlutilityContentBrowserExtensions");
}

#undef LOCTEXT_NAMESPACE
