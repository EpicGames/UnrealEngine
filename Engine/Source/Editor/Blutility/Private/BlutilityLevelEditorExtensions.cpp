// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlutilityLevelEditorExtensions.h"

#include "ActorActionUtility.h"
#include "Algo/AnyOf.h"
#include "AssetRegistry/AssetData.h"
#include "BlutilityMenuExtensions.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "EditorUtilityAssetPrototype.h"
#include "EditorUtilityBlueprint.h"
#include "EditorUtilityWidgetProjectSettings.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "GameFramework/Actor.h"
#include "HAL/PlatformCrt.h"
#include "IAssetTools.h"
#include "LevelEditor.h"
#include "Logging/MessageLog.h"
#include "Misc/NamePermissionList.h"
#include "Modules/ModuleManager.h"
#include "Selection.h"
#include "Templates/Casts.h"
#include "Templates/SharedPointer.h"
#include "Templates/SubclassOf.h"
#include "ToolMenus.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/UObjectIterator.h"

class FUICommandList;

#define LOCTEXT_NAMESPACE "BlutilityLevelEditorExtensions"

void FBlutilityLevelEditorExtensions::InstallHooks()
{
	UToolMenus::RegisterStartupCallback(
		FSimpleMulticastDelegate::FDelegate::CreateStatic(&FBlutilityLevelEditorExtensions::RegisterMenus));
}

void FBlutilityLevelEditorExtensions::RegisterMenus()
{
	// Mark us as the owner of everything we add.
	FToolMenuOwnerScoped OwnerScoped("FBlutilityLevelEditorExtensions");

	UToolMenu* const Menu = UToolMenus::Get()->ExtendMenu("LevelEditor.ActorContextMenu");
	if (!Menu)
	{
		return;
	}

	FToolMenuSection& Section = Menu->FindOrAddSection("ActorOptions");
	Section.AddDynamicEntry("BlutilityLevelEditorExtensions", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
	{
		TArray<AActor*> SelectedActors;
		GEditor->GetSelectedActors()->GetSelectedObjects<AActor>(SelectedActors);

		TMap<TSharedRef<FAssetActionUtilityPrototype>, TSet<int32>> UtilityAndSelectionIndices;

		// Run through the actors to determine if any meet our criteria.
		TArray<AActor*> SupportedActors;
		if (SelectedActors.Num() > 0)
		{
			FMessageLog EditorErrors("EditorErrors");

			auto ProcessActorAction = [&UtilityAndSelectionIndices, &SelectedActors, &SupportedActors, &EditorErrors](const TSharedRef<FAssetActionUtilityPrototype>& ActionUtilityPrototype)
			{
				if (ActionUtilityPrototype->IsLatestVersion())
				{
					TArray<TSoftClassPtr<UObject>> SupportedClassPtrs = ActionUtilityPrototype->GetSupportedClasses();
					if (SupportedClassPtrs.Num() > 0)
					{
						for (AActor* Actor : SelectedActors)
						{
							if (Actor)
							{
								UClass* ActorClass = Actor->GetClass();
								const bool bPassesClassFilter = 
									Algo::AnyOf(SupportedClassPtrs, [ActorClass](TSoftClassPtr<UObject> ClassPtr){ return ActorClass->IsChildOf(ClassPtr.Get()); });

								if (bPassesClassFilter)
								{
									const int32 Index = SupportedActors.AddUnique(Actor);
									UtilityAndSelectionIndices.FindOrAdd(ActionUtilityPrototype).Add(Index);
								}
							}
						}
					}
				}
				else
                {
					const FAssetData& BlutilityAssetData =  ActionUtilityPrototype->GetUtilityBlueprintAsset();
					if (IAssetTools::Get().GetAssetClassPathPermissionList(EAssetClassAction::ViewAsset)->PassesFilter(BlutilityAssetData.AssetClassPath.ToString()))
					{
						EditorErrors.NewPage(LOCTEXT("ScriptedActions", "Scripted Actions"));
						TSharedRef<FTokenizedMessage> ErrorMessage = EditorErrors.Error();
						ErrorMessage->AddToken(FAssetNameToken::Create(BlutilityAssetData.GetObjectPathString(), FText::FromString(BlutilityAssetData.GetObjectPathString())));
						ErrorMessage->AddToken(FTextToken::Create(LOCTEXT("NeedsToBeUpdated", "needs to be re-saved and possibly upgraded.")));
					}
                }
			};

			// Check blueprint utils (we need to load them to query their validity against these assets)
			TArray<FAssetData> UtilAssets;
			FBlutilityMenuExtensions::GetBlutilityClasses(UtilAssets, UActorActionUtility::StaticClass()->GetClassPathName());

			// Process asset based utilities
            for (const FAssetData& UtilAsset : UtilAssets)
            {
            	if (const UClass* ParentClass = UBlueprint::GetBlueprintParentClassFromAssetTags(UtilAsset))
            	{
            		// We only care about UEditorUtilityBlueprint's that are compiling subclasses of UActorActionUtility
            		if (ParentClass->IsChildOf(UActorActionUtility::StaticClass()))
            		{
            			ProcessActorAction(MakeShared<FAssetActionUtilityPrototype>(UtilAsset));
					}
				}
			}

			// Don't warn errors if searching generated classes, since not all utilities may be updated to work with generated classes yet (Must be done piecemeal).
			const UEditorUtilityWidgetProjectSettings* EditorUtilitySettings = GetDefault<UEditorUtilityWidgetProjectSettings>();
			if (!EditorUtilitySettings->bSearchGeneratedClassesForScriptedActions)
			{
				EditorErrors.Notify(LOCTEXT("SomeProblemsWithActorActionUtility", "There were some problems with some ActorActionUtility Blueprints."));
			}
		}

		if (SupportedActors.Num() > 0)
		{
			FBlutilityMenuExtensions::CreateActorBlutilityActionsMenu(InSection, UtilityAndSelectionIndices, SupportedActors);
		}
	}));
}

void FBlutilityLevelEditorExtensions::RemoveHooks()
{
	// Remove our startup delegate in case it's still around.
	UToolMenus::UnRegisterStartupCallback("FBlutilityLevelEditorExtensions");
	// Remove everything we added to UToolMenus.
	UToolMenus::UnregisterOwner("FBlutilityLevelEditorExtensions");
}

#undef LOCTEXT_NAMESPACE
