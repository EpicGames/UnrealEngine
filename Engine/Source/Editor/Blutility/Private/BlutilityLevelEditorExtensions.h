// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Integrate Blutility actions associated with level editor functionality (e.g. Actor editing)
class FBlutilityLevelEditorExtensions
{
public:
	static void InstallHooks();
	static void RegisterMenus();
	static void RemoveHooks();
};
