// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlutilityMenuExtensions.h"

#include "ActorActionUtility.h"
#include "AssetActionUtility.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetDataTagMap.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Blueprint/BlueprintSupport.h"
#include "BlueprintEditorModule.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "DetailsViewArgs.h"
#include "EdGraphSchema_K2.h"
#include "Editor.h"
#include "EditorUtilityAssetPrototype.h"
#include "Editor/EditorEngine.h"
#include "EditorUtilityBlueprint.h"
#include "EditorUtilityWidgetProjectSettings.h"
#include "FrontendFilters.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GameFramework/Actor.h"
#include "GenericPlatform/GenericApplication.h"
#include "IDetailsView.h"
#include "IEditorUtilityExtension.h"
#include "IStructureDetailsView.h"
#include "Input/Reply.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/BasicLayoutWidgetSlot.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/PackageName.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorDelegates.h"
#include "PropertyEditorModule.h"
#include "ScopedTransaction.h"
#include "SlotBase.h"
#include "SPrimaryButton.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Templates/Casts.h"
#include "Templates/SharedPointer.h"
#include "Textures/SlateIcon.h"
#include "Toolkits/IToolkit.h"
#include "ToolMenu.h"
#include "Types/SlateEnums.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/Object.h"
#include "UObject/Package.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/Script.h"
#include "UObject/StructOnScope.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"

class IToolkitHost;

#define LOCTEXT_NAMESPACE "BlutilityMenuExtensions"

namespace BlutilityUtil
{
	/** Mapping of asset property tag aliases that can be used by text searches */
	class FAssetPropertyTagAliases
	{
	public:
		static FAssetPropertyTagAliases& Get()
		{
			static FAssetPropertyTagAliases Singleton;
			return Singleton;
		}

		/** Get the source tag for the given asset data and alias, or none if there is no match */
		FName GetSourceTagFromAlias(const FAssetData& InAssetData, const FName InAlias)
		{
			TSharedPtr<TMap<FName, FName>>& AliasToSourceTagMapping = ClassToAliasTagsMapping.FindOrAdd(InAssetData.AssetClassPath);

			if (!AliasToSourceTagMapping.IsValid())
			{
				static const FName NAME_DisplayName(TEXT("DisplayName"));

				AliasToSourceTagMapping = MakeShared<TMap<FName, FName>>();

				UClass* AssetClass = InAssetData.GetClass();
				if (AssetClass)
				{
					TMap<FName, UObject::FAssetRegistryTagMetadata> AssetTagMetaData;
					AssetClass->GetDefaultObject()->GetAssetRegistryTagMetadata(AssetTagMetaData);

					for (const auto& AssetTagMetaDataPair : AssetTagMetaData)
					{
						if (!AssetTagMetaDataPair.Value.DisplayName.IsEmpty())
						{
							const FName DisplayName = MakeObjectNameFromDisplayLabel(AssetTagMetaDataPair.Value.DisplayName.ToString(), NAME_None);
							AliasToSourceTagMapping->Add(DisplayName, AssetTagMetaDataPair.Key);
						}
					}

					for (const auto& KeyValuePair : InAssetData.TagsAndValues)
					{
						if (FProperty* Field = FindFProperty<FProperty>(AssetClass, KeyValuePair.Key))
						{
							if (Field->HasMetaData(NAME_DisplayName))
							{
								const FName DisplayName = MakeObjectNameFromDisplayLabel(Field->GetMetaData(NAME_DisplayName), NAME_None);
								AliasToSourceTagMapping->Add(DisplayName, KeyValuePair.Key);
							}
						}
					}
				}
			}

			return AliasToSourceTagMapping.IsValid() ? AliasToSourceTagMapping->FindRef(InAlias) : NAME_None;
		}

	private:
		/** Mapping from class name -> (alias -> source) */
		TMap<FTopLevelAssetPath, TSharedPtr<TMap<FName, FName>>> ClassToAliasTagsMapping;
	};
	
	/** Expression context to test the given asset data against the current text filter */
	class FTextFilterExpressionContext : public ITextFilterExpressionContext
	{
	public:
		typedef TRemoveReference<const FAssetData&>::Type* FAssetFilterTypePtr;

		FTextFilterExpressionContext()
			: AssetPtr(nullptr)
			, bIncludeClassName(true)
			, bIncludeAssetPath(false)
			, bIncludeCollectionNames(true)
			, NameKeyName("Name")
			, PathKeyName("Path")
			, ClassKeyName("Class")
			, TypeKeyName("Type")
			, CollectionKeyName("Collection")
			, TagKeyName("Tag")
		{
		}

		void SetAsset(FAssetFilterTypePtr InAsset) 
		{
			AssetPtr = InAsset;

			if (bIncludeAssetPath)
			{
				// Get the full asset path, and also split it so we can compare each part in the filter
				AssetPtr->PackageName.AppendString(AssetFullPath);
				AssetFullPath.ParseIntoArray(AssetSplitPath, TEXT("/"));
				AssetFullPath.ToUpperInline();

				if (bIncludeClassName)
				{
					// Get the full export text path as people sometimes search by copying this (requires class and asset path search to be enabled in order to match)
					AssetPtr->GetExportTextName(AssetExportTextName);
					AssetExportTextName.ToUpperInline();
				}
			}
		}

		const FAssetData* GetAsset() const { return AssetPtr; }

		void ClearAsset()
		{
			AssetPtr = nullptr;
			AssetFullPath.Reset();
			AssetExportTextName.Reset();
			AssetSplitPath.Reset();
			AssetCollectionNames.Reset();
		}

		void SetIncludeClassName(const bool InIncludeClassName)
		{
			bIncludeClassName = InIncludeClassName;
		}

		bool GetIncludeClassName() const
		{
			return bIncludeClassName;
		}

		void SetIncludeAssetPath(const bool InIncludeAssetPath)
		{
			bIncludeAssetPath = InIncludeAssetPath;
		}

		bool GetIncludeAssetPath() const
		{
			return bIncludeAssetPath;
		}

		void SetIncludeCollectionNames(const bool InIncludeCollectionNames)
		{
			bIncludeCollectionNames = InIncludeCollectionNames;
		}

		bool GetIncludeCollectionNames() const
		{
			return bIncludeCollectionNames;
		}

		virtual bool TestBasicStringExpression(const FTextFilterString& InValue, const ETextFilterTextComparisonMode InTextComparisonMode) const override
		{
			if (InValue.CompareName(AssetPtr->AssetName, InTextComparisonMode))
			{
				return true;
			}

			if (bIncludeAssetPath)
			{
				if (InValue.CompareFString(AssetFullPath, InTextComparisonMode))
				{
					return true;
				}

				for (const FString& AssetPathPart : AssetSplitPath)
				{
					if (InValue.CompareFString(AssetPathPart, InTextComparisonMode))
					{
						return true;
					}
				}
			}

			if (bIncludeClassName)
			{
				if (InValue.CompareFString(AssetPtr->AssetClassPath.ToString(), InTextComparisonMode))
				{
					return true;
				}
			}

			if (bIncludeClassName && bIncludeAssetPath)
			{
				// Only test this if we're searching the class name and asset path too, as the exported text contains the type and path in the string
				if (InValue.CompareFString(AssetExportTextName, InTextComparisonMode))
				{
					return true;
				}
			}

			if (bIncludeCollectionNames)
			{
				for (const FName& AssetCollectionName : AssetCollectionNames)
				{
					if (InValue.CompareName(AssetCollectionName, InTextComparisonMode))
					{
						return true;
					}
				}
			}

			return false;
		}

		virtual bool TestComplexExpression(const FName& InKey, const FTextFilterString& InValue, const ETextFilterComparisonOperation InComparisonOperation, const ETextFilterTextComparisonMode InTextComparisonMode) const override
		{
			// Special case for the asset name, as this isn't contained within the asset registry meta-data
			if (InKey == NameKeyName)
			{
				// Names can only work with Equal or NotEqual type tests
				if (InComparisonOperation != ETextFilterComparisonOperation::Equal && InComparisonOperation != ETextFilterComparisonOperation::NotEqual)
				{
					return false;
				}

				const bool bIsMatch = TextFilterUtils::TestBasicStringExpression(AssetPtr->AssetName, InValue, InTextComparisonMode);
				return (InComparisonOperation == ETextFilterComparisonOperation::Equal) ? bIsMatch : !bIsMatch;
			}

			// Special case for the asset path, as this isn't contained within the asset registry meta-data
			if (InKey == PathKeyName)
			{
				// Paths can only work with Equal or NotEqual type tests
				if (InComparisonOperation != ETextFilterComparisonOperation::Equal && InComparisonOperation != ETextFilterComparisonOperation::NotEqual)
				{
					return false;
				}

				// If the comparison mode is partial, then we only need to test the ObjectPath as that contains the other two as sub-strings
				bool bIsMatch = false;
				if (InTextComparisonMode == ETextFilterTextComparisonMode::Partial)
				{
					bIsMatch = TextFilterUtils::TestBasicStringExpression(AssetPtr->GetObjectPathString(), InValue, InTextComparisonMode);
				}
				else
				{
					bIsMatch = TextFilterUtils::TestBasicStringExpression(AssetPtr->GetObjectPathString(), InValue, InTextComparisonMode)
						|| TextFilterUtils::TestBasicStringExpression(AssetPtr->PackageName, InValue, InTextComparisonMode)
						|| TextFilterUtils::TestBasicStringExpression(AssetPtr->PackagePath, InValue, InTextComparisonMode);
				}
				return (InComparisonOperation == ETextFilterComparisonOperation::Equal) ? bIsMatch : !bIsMatch;
			}

			// Special case for the asset type, as this isn't contained within the asset registry meta-data
			if (InKey == ClassKeyName || InKey == TypeKeyName)
			{
				// Class names can only work with Equal or NotEqual type tests
				if (InComparisonOperation != ETextFilterComparisonOperation::Equal && InComparisonOperation != ETextFilterComparisonOperation::NotEqual)
				{
					return false;
				}

				const bool bIsMatch = TextFilterUtils::TestBasicStringExpression(AssetPtr->AssetClassPath.ToString(), InValue, InTextComparisonMode);
				return (InComparisonOperation == ETextFilterComparisonOperation::Equal) ? bIsMatch : !bIsMatch;
			}

			// Special case for collections, as these aren't contained within the asset registry meta-data
			if (InKey == CollectionKeyName || InKey == TagKeyName)
			{
				// Collections can only work with Equal or NotEqual type tests
				if (InComparisonOperation != ETextFilterComparisonOperation::Equal && InComparisonOperation != ETextFilterComparisonOperation::NotEqual)
				{
					return false;
				}

				bool bFoundMatch = false;
				for (const FName& AssetCollectionName : AssetCollectionNames)
				{
					if (TextFilterUtils::TestBasicStringExpression(AssetCollectionName, InValue, InTextComparisonMode))
					{
						bFoundMatch = true;
						break;
					}
				}

				return (InComparisonOperation == ETextFilterComparisonOperation::Equal) ? bFoundMatch : !bFoundMatch;
			}

			// Generic handling for anything in the asset meta-data
			{
				auto GetMetaDataValue = [this, &InKey](FString& OutMetaDataValue) -> bool
				{
					// Check for a literal key
					if (AssetPtr->GetTagValue(InKey, OutMetaDataValue))
					{
						return true;
					}

					// Check for an alias key
					const FName LiteralKey = FAssetPropertyTagAliases::Get().GetSourceTagFromAlias(*AssetPtr, InKey);
					if (!LiteralKey.IsNone() && AssetPtr->GetTagValue(LiteralKey, OutMetaDataValue))
					{
						return true;
					}

					return false;
				};

				FString MetaDataValue;
				if (GetMetaDataValue(MetaDataValue))
				{
					return TextFilterUtils::TestComplexExpression(MetaDataValue, InValue, InComparisonOperation, InTextComparisonMode);
				}
			}

			return false;
		}

	private:
		/** Pointer to the asset we're currently filtering */
		FAssetFilterTypePtr AssetPtr;

		/** Full path of the current asset */
		FString AssetFullPath;

		/** The export text name of the current asset */
		FString AssetExportTextName;

		/** Split path of the current asset */
		TArray<FString> AssetSplitPath;

		/** Names of the collections that the current asset is in */
		TArray<FName> AssetCollectionNames;

		/** Are we supposed to include the class name in our basic string tests? */
		bool bIncludeClassName;

		/** Search inside the entire asset path? */
		bool bIncludeAssetPath;

		/** Search collection names? */
		bool bIncludeCollectionNames;

		/** Keys used by TestComplexExpression */
		const FName NameKeyName;
		const FName PathKeyName;
		const FName ClassKeyName;
		const FName TypeKeyName;
		const FName CollectionKeyName;
		const FName TagKeyName;
	};	
}

/** Dialog widget used to display function properties */
class SFunctionParamDialog : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SFunctionParamDialog) {}

	/** Text to display on the "OK" button */
	SLATE_ARGUMENT(FText, OkButtonText)

	/** Tooltip text for the "OK" button */
	SLATE_ARGUMENT(FText, OkButtonTooltipText)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TWeakPtr<SWindow> InParentWindow, TSharedRef<FStructOnScope> InStructOnScope, FName HiddenPropertyName)
	{
		bOKPressed = false;

		// Initialize details view
		FDetailsViewArgs DetailsViewArgs;
		{
			DetailsViewArgs.bAllowSearch = false;
			DetailsViewArgs.bHideSelectionTip = true;
			DetailsViewArgs.bLockable = false;
			DetailsViewArgs.bSearchInitialKeyFocus = true;
			DetailsViewArgs.bUpdatesFromSelection = false;
			DetailsViewArgs.bShowOptions = false;
			DetailsViewArgs.bShowModifiedPropertiesOption = false;
			DetailsViewArgs.bShowObjectLabel = false;
			DetailsViewArgs.bForceHiddenPropertyVisibility = true;
			DetailsViewArgs.bShowScrollBar = false;
		}
	
		FStructureDetailsViewArgs StructureViewArgs;
		{
			StructureViewArgs.bShowObjects = true;
			StructureViewArgs.bShowAssets = true;
			StructureViewArgs.bShowClasses = true;
			StructureViewArgs.bShowInterfaces = true;
		}

		FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		TSharedRef<IStructureDetailsView> StructureDetailsView = PropertyEditorModule.CreateStructureDetailView(DetailsViewArgs, StructureViewArgs, InStructOnScope);

		// Hide any property that has been marked as such
		StructureDetailsView->GetDetailsView()->SetIsPropertyVisibleDelegate(FIsPropertyVisible::CreateLambda([HiddenPropertyName](const FPropertyAndParent& InPropertyAndParent)
		{
			if (InPropertyAndParent.Property.GetFName() == HiddenPropertyName)
			{
				return false;
			}

			if (InPropertyAndParent.Property.HasAnyPropertyFlags(CPF_Parm))
			{
				return true;
			}

			for (const FProperty* Parent : InPropertyAndParent.ParentProperties)
			{
				if (Parent->HasAnyPropertyFlags(CPF_Parm))
				{
					return true;
				}
			}

			return false;
		}));

		StructureDetailsView->GetDetailsView()->ForceRefresh();

		ChildSlot
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SNew(SScrollBox)
				+SScrollBox::Slot()
				[
					StructureDetailsView->GetWidget().ToSharedRef()
				]
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				.Padding(16.0f)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Right)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SPrimaryButton)
						.Text(InArgs._OkButtonText)
						.ToolTipText(InArgs._OkButtonTooltipText)
						.OnClicked_Lambda([this, InParentWindow, InArgs]()
						{
							if(InParentWindow.IsValid())
							{
								InParentWindow.Pin()->RequestDestroyWindow();
							}
							bOKPressed = true;
							return FReply::Handled(); 
						})
					]
					+SHorizontalBox::Slot()
					.Padding(FMargin(8.0f, 0.0f, 0.0f, 0.0f))
					.AutoWidth()
					[
						SNew(SButton)
						.TextStyle(FAppStyle::Get(), "DialogButtonText")
						.Text(LOCTEXT("Cancel", "Cancel"))
						.OnClicked_Lambda([InParentWindow]()
						{
							if(InParentWindow.IsValid())
							{
								InParentWindow.Pin()->RequestDestroyWindow();
							}
							return FReply::Handled(); 
						})
					]
				]
			]
		];
	}

	bool bOKPressed;
};

void FBlutilityMenuExtensions::GetBlutilityClasses(TArray<FAssetData>& OutAssets, FTopLevelAssetPath InClassName)
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

	// Get class names
	TArray<FTopLevelAssetPath> BaseNames;
	BaseNames.Add(InClassName);
	TSet<FTopLevelAssetPath> DerivedClasses;
	AssetRegistry.GetDerivedClassNames(BaseNames, TSet<FTopLevelAssetPath>(), DerivedClasses);
	
	// Now get all UEditorUtilityBlueprint assets
	FARFilter Filter;
	Filter.ClassPaths.Add(UEditorUtilityBlueprint::StaticClass()->GetClassPathName());
	Filter.bRecursiveClasses = true;
	Filter.bRecursivePaths = true;

	TArray<FAssetData> AssetList;
	AssetRegistry.GetAssets(Filter, AssetList);

	// Check each asset to see if it matches our type
	for (const FAssetData& Asset : AssetList)
	{
		// Abstract or deprecated blutilities should not be included.
		const EClassFlags BPFlags = static_cast<EClassFlags>(Asset.GetTagValueRef<uint32>(FBlueprintTags::ClassFlags));
		if (EnumHasAnyFlags(BPFlags, CLASS_Abstract | CLASS_Deprecated))
		{
			continue;
		}
		
		FAssetDataTagMapSharedView::FFindTagResult Result = Asset.TagsAndValues.FindTag(FBlueprintTags::GeneratedClassPath);
		if (Result.IsSet())
		{
			// If it's a menu extension blutility, don't include ones from other peoples developer folders, just your own.
			const FString& AssetPath = FPackageName::LongPackageNameToFilename(Asset.PackagePath.ToString());
			if (FPaths::IsUnderDirectory(AssetPath, FPaths::GameDevelopersDir()))
			{
				if (!FPaths::IsUnderDirectory(AssetPath, FPaths::GameUserDeveloperDir()))
				{
					continue;
				}
			}
			
			const FTopLevelAssetPath ClassObjectPath(FPackageName::ExportTextPathToObjectPath(Result.GetValue()));

			if (DerivedClasses.Contains(ClassObjectPath))
			{
				OutAssets.Add(Asset);
			}
		}
	}

	const UEditorUtilityWidgetProjectSettings* EditorUtilitySettings = GetDefault<UEditorUtilityWidgetProjectSettings>();

	if (EditorUtilitySettings->bSearchGeneratedClassesForScriptedActions)
	{
		auto FilterAssets = [&AssetRegistry, &OutAssets](const FNamePermissionList& PermissionList)
		{
			FARFilter GeneratedFilter;
			GeneratedFilter.ClassPaths.Add(UBlueprintGeneratedClass::StaticClass()->GetClassPathName());
			GeneratedFilter.bRecursiveClasses = true;
			GeneratedFilter.bRecursivePaths = true;

			TArray<FAssetData> GeneratedAssetList;
			AssetRegistry.GetAssets(GeneratedFilter, GeneratedAssetList);

			for (const FAssetData& Asset : GeneratedAssetList)
			{
				// Abstract or deprecated blutilities should not be included.
				const EClassFlags BPFlags = static_cast<EClassFlags>(Asset.GetTagValueRef<uint32>(FBlueprintTags::ClassFlags));
				if (EnumHasAnyFlags(BPFlags, CLASS_Abstract | CLASS_Deprecated))
				{
					continue;
				}

				if (PermissionList.PassesFilter(FName(Asset.GetObjectPathString())))
				{
					OutAssets.Add(Asset);
				}
			}
		};

		if (InClassName == UAssetActionUtility::StaticClass()->GetClassPathName())
		{
			FilterAssets(EditorUtilitySettings->GetAllowedEditorUtilityAssetActions());
		}
		else if (InClassName == UActorActionUtility::StaticClass()->GetClassPathName())
		{
			FilterAssets(EditorUtilitySettings->GetAllowedEditorUtilityActorActions());
		}
	}
}

void FBlutilityMenuExtensions::CreateActorBlutilityActionsMenu(FToolMenuSection& InSection, TMap<TSharedRef<FAssetActionUtilityPrototype>, TSet<int32>> Utils, const TArray<AActor*> SelectedSupportedActors)
{
	CreateBlutilityActionsMenu<AActor*>(InSection, Utils,
		"ScriptedActorActions",
		LOCTEXT("ScriptedActorActions", "Scripted Actor Actions"),
		LOCTEXT("ScriptedActorActionsTooltip", "Scripted actions available for the selected actors"),
		[](const FProperty* Property) -> bool
		{
			if (const FObjectProperty* ObjectProperty = CastField<const FObjectProperty>(Property))
			{
				return ObjectProperty->PropertyClass == AActor::StaticClass();
			}

			return false;
		},
		SelectedSupportedActors,
		"Actors.ScripterActorActions"
	);
}

void FBlutilityMenuExtensions::CreateAssetBlutilityActionsMenu(FToolMenuSection& InSection, TMap<TSharedRef<FAssetActionUtilityPrototype>, TSet<int32>> Utils, const TArray<FAssetData> SelectedSupportedAssets)
{
	CreateBlutilityActionsMenu<FAssetData>(InSection, Utils,
		"ScriptedAssetActions",
		LOCTEXT("ScriptedAssetActions", "Scripted Asset Actions"),
		LOCTEXT("ScriptedAssetActionsTooltip", "Scripted actions available for the selected assets"),
		[](const FProperty* Property) -> bool
		{
			const FFieldClass* ClassOfProperty = Property->GetClass();
			if (ClassOfProperty == FStructProperty::StaticClass())
			{
				const FStructProperty* StructProperty = CastField<const FStructProperty>(Property);
				return StructProperty->Struct->GetName() == TEXT("AssetData");
			}

			return false;
		},
		SelectedSupportedAssets,
		"Actors.ScripterActorActions"
	);	
}

void FBlutilityMenuExtensions::CreateObjectBlutilityActionsMenu(
	FMenuBuilder& MenuBuilder,
	const FMenuParams& MenuParams,
	const FObjectActionParams& ObjectActionParams)
{
	if (MenuParams.bAsSubMenu)
	{
		CreateBlutilityActionsMenu<UObject*>(MenuBuilder,
			ObjectActionParams.Utils,
			MenuParams.Label,
			MenuParams.ToolTip,
			ObjectActionParams.IsValidPropertyFunc,
			ObjectActionParams.Objects,
			MenuParams.IconName
		);
	}
	else
	{
		CreateBlulityActionsSection<UObject*>(MenuBuilder, ObjectActionParams);
	}
}

void FBlutilityMenuExtensions::OpenEditorForUtility(const FFunctionAndUtil& FunctionAndUtil)
{
	// Edit the script if we have shift held down
	if (UBlueprint* Blueprint = Cast<UBlueprint>(Cast<UObject>(FunctionAndUtil.Util->LoadUtilityAsset())->GetClass()->ClassGeneratedBy))
	{
		if (IAssetEditorInstance* AssetEditor = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(Blueprint, true))
		{
			check(AssetEditor->GetEditorName() == TEXT("BlueprintEditor"));
			IBlueprintEditor* BlueprintEditor = static_cast<IBlueprintEditor*>(AssetEditor);
			BlueprintEditor->JumpToHyperlink(FunctionAndUtil.GetFunction(), false);
		}
		else
		{
			FBlueprintEditorModule& BlueprintEditorModule = FModuleManager::LoadModuleChecked<FBlueprintEditorModule>("Kismet");
			TSharedRef<IBlueprintEditor> BlueprintEditor = BlueprintEditorModule.CreateBlueprintEditor(EToolkitMode::Standalone, TSharedPtr<IToolkitHost>(), Blueprint, false);
			BlueprintEditor->JumpToHyperlink(FunctionAndUtil.GetFunction(), false);
		}
	}
}

void FBlutilityMenuExtensions::ExtractFunctions(const TMap<TSharedRef<FAssetActionUtilityPrototype>, TSet<int32>>& Utils, TMap<FString, TArray<FFunctionAndUtil>>& OutCategoryFunctions)
{
	// Find the exposed functions available in each class, making sure to not list shared functions from a parent class more than once
	for (TPair<TSharedRef<FAssetActionUtilityPrototype>, TSet<int32>> UtilitySelectionPair : Utils)
	{
		const TSharedRef<FAssetActionUtilityPrototype>& Util = UtilitySelectionPair.Key;

		TArray<FBlutilityFunctionData> FunctionDatas = Util->GetCallableFunctions();
		for (const FBlutilityFunctionData& FunctionData : FunctionDatas)
		{
			TArray<FFunctionAndUtil>& Functions = OutCategoryFunctions.FindOrAdd(FunctionData.Category);
			Functions.AddUnique(FFunctionAndUtil(FunctionData, Util, UtilitySelectionPair.Value));
		}
	}

	for (TPair<FString, TArray<FFunctionAndUtil>>& CategoryFunctionPair : OutCategoryFunctions)
	{
		// Sort the functions by name
		CategoryFunctionPair.Value.Sort([](const FFunctionAndUtil& A, const FFunctionAndUtil& B) { return A.FunctionData.Name.LexicalLess(B.FunctionData.Name); });
	}
}

template<typename SelectionType>
void FBlutilityMenuExtensions::CreateBlutilityActionsMenu(FMenuBuilder& MenuBuilder, TMap<TSharedRef<FAssetActionUtilityPrototype>, TSet<int32>> Utils, const FText& MenuLabel, const FText& MenuToolTip, TFunction<bool(const FProperty * Property)> IsValidPropertyType, const TArray<SelectionType> Selection, const FName& IconName)
{
	TMap<FString, TArray<FFunctionAndUtil>> CategoryFunctions;
	ExtractFunctions(Utils, CategoryFunctions);
	
	auto AddFunctionEntries = [Selection, IsValidPropertyType](const TArray<FFunctionAndUtil>& FunctionUtils)
	{
		BlutilityUtil::FTextFilterExpressionContext TextFilterContext;
		
		FTextFilterExpressionEvaluator TextFilterExpressionEvaluator(ETextFilterExpressionEvaluatorMode::Complex);
		TArray<FMenuEntryParams> GeneratedMenuEntryParams;
		for (const FFunctionAndUtil& FunctionAndUtil : FunctionUtils)
		{
			bool PassesFilterCondition = true;
			bool bShowInMenu = true;
			FString FilterFailureMessage;
			
			if constexpr ( std::is_same_v<FAssetData, SelectionType> )
			{
				TArray<FAssetActionSupportCondition> Conditions = FunctionAndUtil.Util->GetAssetActionSupportConditions();

				for (const FAssetActionSupportCondition& Condition : Conditions)
				{
					TextFilterExpressionEvaluator.SetFilterText(FText::FromString(Condition.Filter));
				
					for (const int32& SelectionIndex : FunctionAndUtil.SelectionIndices)
					{
						const auto SelectedAsset = Selection[SelectionIndex];
						TextFilterContext.SetAsset(&SelectedAsset);
						if (!TextFilterExpressionEvaluator.TestTextFilter(TextFilterContext))
						{
							PassesFilterCondition = false;
							FilterFailureMessage = Condition.FailureReason;
							break;
						}
					}

					if (!PassesFilterCondition)
					{
						bShowInMenu = Condition.bShowInMenuIfFilterFails;
						break;
					}
				}
			}

			if (!PassesFilterCondition && !bShowInMenu)
			{
				continue;
			}
			
			FText TooltipText;

			if (FunctionAndUtil.Util->GetUtilityBlueprintAsset().AssetClassPath == UBlueprintGeneratedClass::StaticClass()->GetClassPathName())
			{
				TooltipText = FText::Format(LOCTEXT("AssetUtilTooltip", "{0}\n\n(Click to execute)"), FunctionAndUtil.FunctionData.TooltipText);
			}
			else if (FilterFailureMessage.IsEmpty())
			{
				TooltipText = FText::Format(LOCTEXT("AssetUtilTooltipFormat", "{0}\n\n(Shift-click to edit script)"), FunctionAndUtil.FunctionData.TooltipText);	
			}
			else
			{
				TooltipText = FText::Format(LOCTEXT("AssetUtilTooltipWithErrorFormat", "{0}\n\n({1})\n\n(Shift-click to edit script)"), FunctionAndUtil.FunctionData.TooltipText, FText::FromString(FilterFailureMessage));
			}

			FMenuEntryParams MenuEntryParams;
			MenuEntryParams.LabelOverride = FunctionAndUtil.FunctionData.NameText;
			MenuEntryParams.ToolTipOverride = TooltipText;
			MenuEntryParams.IconOverride = FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.Event_16x");
			MenuEntryParams.DirectActions = FUIAction(FExecuteAction::CreateLambda([FunctionAndUtil, Selection, IsValidPropertyType]
				{
					if (FSlateApplication::Get().GetModifierKeys().IsShiftDown())
					{
						OpenEditorForUtility(FunctionAndUtil);
					}
					else
					{
						// We dont run this on the CDO, as bad things could occur!
						UObject* TempObject = NewObject<UObject>(GetTransientPackage(), Cast<UObject>(FunctionAndUtil.Util->LoadUtilityAsset())->GetClass());
						TempObject->AddToRoot(); // Some Blutility actions might run GC so the TempObject needs to be rooted to avoid getting destroyed

						UFunction* Function = FunctionAndUtil.GetFunction();
						
						if (Function->NumParms > 0)
						{
							// Create a parameter struct and fill in defaults
							TSharedRef<FStructOnScope> FuncParams = MakeShared<FStructOnScope>(Function);

							FProperty* FirstParamProperty = nullptr;

							int32 ParameterIndex = 0;
							for (TFieldIterator<FProperty> It(Function); It&& It->HasAnyPropertyFlags(CPF_Parm); ++It)
							{
								FString Defaults;
								if (UEdGraphSchema_K2::FindFunctionParameterDefaultValue(Function, *It, Defaults))
								{
									It->ImportText_Direct(*Defaults, It->ContainerPtrToValuePtr<uint8>(FuncParams->GetStructMemory()), nullptr, PPF_None);
								}

								// Check to see if the first parameter matches the selection object type, in that case we can directly forward the selection to it
								if (ParameterIndex == 0 && IsValidPropertyType(*It))
								{
									FirstParamProperty = *It;
								}

								++ParameterIndex;
							}

							bool bApply = true;

							if (!FirstParamProperty || ParameterIndex > 1)
							{
								// pop up a dialog to input params to the function
								TSharedRef<SWindow> Window = SNew(SWindow)
									.Title(Function->GetDisplayNameText())
									.ClientSize(FVector2D(400, 200))
									.SupportsMinimize(false)
									.SupportsMaximize(false);

								TSharedPtr<SFunctionParamDialog> Dialog;
								Window->SetContent(
									SAssignNew(Dialog, SFunctionParamDialog, Window, FuncParams, FirstParamProperty ? FirstParamProperty->GetFName() : NAME_None)
									.OkButtonText(LOCTEXT("OKButton", "OK"))
									.OkButtonTooltipText(Function->GetToolTipText()));

								GEditor->EditorAddModalWindow(Window);
								bApply = Dialog->bOKPressed;
							}


							if (bApply)
							{
								FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "BlutilityAction", "Blutility Action"));
								FEditorScriptExecutionGuard ScriptGuard;
								const bool bForwardUserSelection = FirstParamProperty != nullptr;
								if (bForwardUserSelection)
								{
									// For each user-select asset forward the selection object into the function first's parameter (if it matches)
									const FString Path = FirstParamProperty->GetPathName(Function);

									// Ensure we only process selection objects that are valid for this function/utility
									for (const int32& SelectionIndex : FunctionAndUtil.SelectionIndices)
									{
										const auto SelectedAsset = Selection[SelectionIndex];
										FirstParamProperty->CopySingleValue(FirstParamProperty->ContainerPtrToValuePtr<uint8>(FuncParams->GetStructMemory()), &SelectedAsset);
										TempObject->ProcessEvent(Function, FuncParams->GetStructMemory());
									}
								}
								else
								{
									// User is expected to manage the asset selection on its own
									TempObject->ProcessEvent(Function, FuncParams->GetStructMemory());
								}
							}
						}
						else
						{
							FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "BlutilityAction", "Blutility Action"));
							FEditorScriptExecutionGuard ScriptGuard;
							TempObject->ProcessEvent(Function, nullptr);
						}

						TempObject->RemoveFromRoot();
					}
				}),
				FCanExecuteAction::CreateLambda([PassesFilterCondition]()
				{
					return PassesFilterCondition;
				})
			);

			GeneratedMenuEntryParams.Emplace(MenuEntryParams);
		}

		return GeneratedMenuEntryParams;
	};

	// Add a menu item for each function
	if (CategoryFunctions.Num() > 0)
	{
		MenuBuilder.AddSubMenu(
			MenuLabel,
			MenuToolTip,
			FNewMenuDelegate::CreateLambda([CategoryFunctions, AddFunctionEntries](FMenuBuilder& InMenuBuilder)
			{
				TArray<FString> CategoryNames;
				CategoryFunctions.GenerateKeyArray(CategoryNames);
				CategoryNames.Remove(FString());
				CategoryNames.Sort();
				
				// Add functions belong to the same category to a sub-menu
				for (const FString& CategoryName : CategoryNames)
				{
					const TArray<FMenuEntryParams> GeneratedMenuEntries = AddFunctionEntries(CategoryFunctions.FindChecked(CategoryName));
					if (GeneratedMenuEntries.Num() > 0)
					{
						InMenuBuilder.AddSubMenu(FText::FromString(CategoryName), FText::FromString(CategoryName),
							FNewMenuDelegate::CreateLambda([GeneratedMenuEntries](FMenuBuilder& InSubMenuBuilder)
								{
									for (const FMenuEntryParams& MenuParams : GeneratedMenuEntries)
									{
										InSubMenuBuilder.AddMenuEntry(MenuParams);
									}
								})
						);
					}
				}

				// Non-categorized functions
				const TArray<FFunctionAndUtil>* DefaultCategoryFunctionsPtr = CategoryFunctions.Find(FString());
				if (DefaultCategoryFunctionsPtr)
				{
					for (const FMenuEntryParams& MenuParams : AddFunctionEntries(*DefaultCategoryFunctionsPtr))
					{
						InMenuBuilder.AddMenuEntry(MenuParams);
					}
				}
			}),
			false,
			FSlateIcon(FAppStyle::GetAppStyleSetName(), IconName)
		);
	}
}

template<typename SelectionType>
void FBlutilityMenuExtensions::CreateBlulityActionsSection(FMenuBuilder& InMenuBuilder, const FObjectActionParams& InObjectActionParams)
{
	TMap<FString, TArray<FFunctionAndUtil>> CategoryFunctions;
	ExtractFunctions(InObjectActionParams.Utils, CategoryFunctions);
	
	auto AddFunctionEntries = [Selection = InObjectActionParams.Objects, IsValidPropertyType = InObjectActionParams.IsValidPropertyFunc](
		const TArray<FFunctionAndUtil>& FunctionUtils)
	{
		BlutilityUtil::FTextFilterExpressionContext TextFilterContext;
		
		FTextFilterExpressionEvaluator TextFilterExpressionEvaluator(ETextFilterExpressionEvaluatorMode::Complex);
		TArray<FMenuEntryParams> GeneratedMenuEntryParams;
		for (const FFunctionAndUtil& FunctionAndUtil : FunctionUtils)
		{
			bool PassesFilterCondition = true;
			bool bShowInMenu = true;
			FString FilterFailureMessage;
			
			if constexpr ( std::is_same_v<FAssetData, SelectionType> )
			{
				TArray<FAssetActionSupportCondition> Conditions = FunctionAndUtil.Util->GetAssetActionSupportConditions();

				for (const FAssetActionSupportCondition& Condition : Conditions)
				{
					TextFilterExpressionEvaluator.SetFilterText(FText::FromString(Condition.Filter));
				
					for (const int32& SelectionIndex : FunctionAndUtil.SelectionIndices)
					{
						const auto SelectedAsset = Cast<SelectionType>(Selection[SelectionIndex]);
						TextFilterContext.SetAsset(&SelectedAsset);
						if (!TextFilterExpressionEvaluator.TestTextFilter(TextFilterContext))
						{
							PassesFilterCondition = false;
							FilterFailureMessage = Condition.FailureReason;
							break;
						}
					}

					if (!PassesFilterCondition)
					{
						bShowInMenu = Condition.bShowInMenuIfFilterFails;
						break;
					}
				}
			}

			if (!PassesFilterCondition && !bShowInMenu)
			{
				continue;
			}
			
			FText TooltipText;

			if (FunctionAndUtil.Util->GetUtilityBlueprintAsset().AssetClassPath == UBlueprintGeneratedClass::StaticClass()->GetClassPathName())
			{
				TooltipText = FText::Format(LOCTEXT("AssetUtilTooltip", "{0}\n\n(Click to execute)"), FunctionAndUtil.FunctionData.TooltipText);
			}
			else if (FilterFailureMessage.IsEmpty())
			{
				TooltipText = FText::Format(LOCTEXT("AssetUtilTooltipFormat", "{0}\n\n(Shift-click to edit script)"), FunctionAndUtil.FunctionData.TooltipText);	
			}
			else
			{
				TooltipText = FText::Format(LOCTEXT("AssetUtilTooltipWithErrorFormat", "{0}\n\n({1})\n\n(Shift-click to edit script)"), FunctionAndUtil.FunctionData.TooltipText, FText::FromString(FilterFailureMessage));
			}

			FMenuEntryParams MenuEntryParams;
			MenuEntryParams.LabelOverride = FunctionAndUtil.FunctionData.NameText;
			MenuEntryParams.ToolTipOverride = TooltipText;
			MenuEntryParams.IconOverride = FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.Event_16x");
			MenuEntryParams.DirectActions = FUIAction(FExecuteAction::CreateLambda([FunctionAndUtil, Selection, IsValidPropertyType]
				{
					if (FSlateApplication::Get().GetModifierKeys().IsShiftDown())
					{
						OpenEditorForUtility(FunctionAndUtil);
					}
					else
					{
						// We dont run this on the CDO, as bad things could occur!
						UObject* UtilityObject = Cast<UObject>(FunctionAndUtil.Util->LoadUtilityAsset());
						UFunction* Function = FunctionAndUtil.GetFunction();
						if (UtilityObject && Function)
						{
							UObject* TempObject = NewObject<UObject>(GetTransientPackage(), UtilityObject->GetClass());
							TempObject->AddToRoot(); // Some Blutility actions might run GC so the TempObject needs to be rooted to avoid getting destroyed

							if (Function->NumParms > 0)
							{
								// Create a parameter struct and fill in defaults
								TSharedRef<FStructOnScope> FuncParams = MakeShared<FStructOnScope>(Function);

								FProperty* FirstParamProperty = nullptr;

								int32 ParameterIndex = 0;
								for (TFieldIterator<FProperty> It(Function); It&& It->HasAnyPropertyFlags(CPF_Parm); ++It)
								{
									FString Defaults;
									if (UEdGraphSchema_K2::FindFunctionParameterDefaultValue(Function, *It, Defaults))
									{
										It->ImportText_Direct(*Defaults, It->ContainerPtrToValuePtr<uint8>(FuncParams->GetStructMemory()), nullptr, PPF_None);
									}

									// Check to see if the first parameter matches the selection object type, in that case we can directly forward the selection to it
									if (ParameterIndex == 0 && IsValidPropertyType(*It))
									{
										FirstParamProperty = *It;
									}

									++ParameterIndex;
							   }

								bool bApply = true;

								if (!FirstParamProperty || ParameterIndex > 1)
								{
									// pop up a dialog to input params to the function
									TSharedRef<SWindow> Window = SNew(SWindow)
										.Title(Function->GetDisplayNameText())
										.ClientSize(FVector2D(400, 200))
										.SupportsMinimize(false)
										.SupportsMaximize(false);

									TSharedPtr<SFunctionParamDialog> Dialog;
									Window->SetContent(
										SAssignNew(Dialog, SFunctionParamDialog, Window, FuncParams, FirstParamProperty ? FirstParamProperty->GetFName() : NAME_None)
										.OkButtonText(LOCTEXT("OKButton", "OK"))
										.OkButtonTooltipText(Function->GetToolTipText()));

									GEditor->EditorAddModalWindow(Window);
									bApply = Dialog->bOKPressed;
								}

								if (bApply)
								{
									FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "BlutilityAction", "Blutility Action"));
									FEditorScriptExecutionGuard ScriptGuard;
									const bool bForwardUserSelection = FirstParamProperty != nullptr;
									if (bForwardUserSelection)
									{
										// For each user-select asset forward the selection object into the function first's parameter (if it matches)
										const FString Path = FirstParamProperty->GetPathName(Function);

										// Ensure we only process selection objects that are valid for this function/utility
										for (const int32& SelectionIndex : FunctionAndUtil.SelectionIndices)
										{
											const auto SelectedAsset = Selection[SelectionIndex];
											FirstParamProperty->CopySingleValue(
												FirstParamProperty->ContainerPtrToValuePtr<uint8>(FuncParams->GetStructMemory()), &SelectedAsset);
											TempObject->ProcessEvent(Function, FuncParams->GetStructMemory());
										}
									}
									else
									{
										// User is expected to manage the asset selection on its own
										TempObject->ProcessEvent(Function, FuncParams->GetStructMemory());
									}
								}
							}
							else
							{
								FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "BlutilityAction", "Blutility Action"));
								FEditorScriptExecutionGuard ScriptGuard;
								TempObject->ProcessEvent(Function, nullptr);
							}
							
							TempObject->RemoveFromRoot();
						}
					}
				}),
				FCanExecuteAction::CreateLambda([PassesFilterCondition]()
				{
					return PassesFilterCondition;
				})
			);

			GeneratedMenuEntryParams.Emplace(MenuEntryParams);
		}

		return GeneratedMenuEntryParams;
	};

	// Add a menu item for each function
	if (CategoryFunctions.Num() > 0)
	{
		TArray<FString> CategoryNames;
		CategoryFunctions.GenerateKeyArray(CategoryNames);
		CategoryNames.Remove(FString());
		CategoryNames.Sort();
			
		for (const FString& CategoryName : CategoryNames)
		{
			const TArray<FMenuEntryParams> GeneratedMenuEntries = AddFunctionEntries(CategoryFunctions.FindChecked(CategoryName));
			if (!GeneratedMenuEntries.IsEmpty())
			{
				InMenuBuilder.BeginSection(*CategoryName, FText::FromString(CategoryName));
				{
					for (const FMenuEntryParams& MenuParams : GeneratedMenuEntries)
					{
						InMenuBuilder.AddMenuEntry(MenuParams);
					}
				}
				InMenuBuilder.EndSection();
			}
		}

		// Non-categorized functions
		if (const TArray<FFunctionAndUtil>* DefaultCategoryFunctionsPtr = CategoryFunctions.Find(FString()))
		{
			const TArray<FMenuEntryParams> GeneratedMenuEntries = AddFunctionEntries(*DefaultCategoryFunctionsPtr);
			if (!GeneratedMenuEntries.IsEmpty())
			{
				for (const FMenuEntryParams& MenuParams : GeneratedMenuEntries)
				{
					InMenuBuilder.AddMenuEntry(MenuParams);
				}
			}
		}
	}
}

template<typename SelectionType>
void FBlutilityMenuExtensions::CreateBlutilityActionsMenu(FToolMenuSection& InSection, TMap<TSharedRef<FAssetActionUtilityPrototype>, TSet<int32>> Utils, const FName& MenuName, const FText& MenuLabel, const FText& MenuToolTip, TFunction<bool(const FProperty * Property)> IsValidPropertyType, const TArray<SelectionType> Selection, const FName& IconName)
{
	TMap<FString, TArray<FFunctionAndUtil>> CategoryFunctions;
	ExtractFunctions(Utils, CategoryFunctions);
	
	auto AddFunctionEntries = [Selection, IsValidPropertyType](const TArray<FFunctionAndUtil>& FunctionUtils) -> TArray<FToolMenuEntry>
	{
		BlutilityUtil::FTextFilterExpressionContext TextFilterContext;
		
		FTextFilterExpressionEvaluator TextFilterExpressionEvaluator(ETextFilterExpressionEvaluatorMode::Complex);
		TArray<FToolMenuEntry> GeneratedMenuEntries;

		for (const FFunctionAndUtil& FunctionAndUtil : FunctionUtils)
		{
			bool PassesFilterCondition = true;
			bool bShowInMenu = true;
			FString FilterFailureMessage;
			
			if constexpr ( std::is_same_v<FAssetData, SelectionType> )
			{
				TArray<FAssetActionSupportCondition> Conditions = FunctionAndUtil.Util->GetAssetActionSupportConditions();

				for (const FAssetActionSupportCondition& Condition : Conditions)
				{
					TextFilterExpressionEvaluator.SetFilterText(FText::FromString(Condition.Filter));
				
					for (const int32& SelectionIndex : FunctionAndUtil.SelectionIndices)
					{
						const auto SelectedAsset = Selection[SelectionIndex];
						TextFilterContext.SetAsset(&SelectedAsset);
						if (!TextFilterExpressionEvaluator.TestTextFilter(TextFilterContext))
						{
							PassesFilterCondition = false;
							FilterFailureMessage = Condition.FailureReason;
							break;
						}
					}

					if (!PassesFilterCondition)
					{
						bShowInMenu = Condition.bShowInMenuIfFilterFails;
						break;
					}
				}
			}

			if (!PassesFilterCondition && !bShowInMenu)
			{
				continue;
			}
			
			FText TooltipText;

			if (FunctionAndUtil.Util->GetUtilityBlueprintAsset().AssetClassPath == UBlueprintGeneratedClass::StaticClass()->GetClassPathName())
			{
				TooltipText = FText::Format(LOCTEXT("AssetUtilTooltip", "{0}\n\n(Click to execute)"), FunctionAndUtil.FunctionData.TooltipText);
			}
			else if (FilterFailureMessage.IsEmpty())
			{
				TooltipText = FText::Format(LOCTEXT("AssetUtilTooltipFormat", "{0}\n\n(Shift-click to edit script)"), FunctionAndUtil.FunctionData.TooltipText);	
			}
			else
			{
				TooltipText = FText::Format(LOCTEXT("AssetUtilTooltipWithErrorFormat", "{0}\n\n({1})\n\n(Shift-click to edit script)"), FunctionAndUtil.FunctionData.TooltipText, FText::FromString(FilterFailureMessage));
			}

			FExecuteAction ExecuteAction = FExecuteAction::CreateLambda([FunctionAndUtil, Selection, IsValidPropertyType]
			{
				if (FSlateApplication::Get().GetModifierKeys().IsShiftDown())
				{
					OpenEditorForUtility(FunctionAndUtil);
				}
				else
				{
					// We dont run this on the CDO, as bad things could occur!
					UObject* TempObject = NewObject<UObject>(GetTransientPackage(), Cast<UObject>(FunctionAndUtil.Util->LoadUtilityAsset())->GetClass());
					TempObject->AddToRoot(); // Some Blutility actions might run GC so the TempObject needs to be rooted to avoid getting destroyed

					UFunction* Function = FunctionAndUtil.GetFunction();
					
					if (Function->NumParms > 0)
					{
						// Create a parameter struct and fill in defaults
						TSharedRef<FStructOnScope> FuncParams = MakeShared<FStructOnScope>(Function);

						FProperty* FirstParamProperty = nullptr;

						int32 ParameterIndex = 0;
						for (TFieldIterator<FProperty> It(Function); It&& It->HasAnyPropertyFlags(CPF_Parm); ++It)
						{
							FString Defaults;
							if (UEdGraphSchema_K2::FindFunctionParameterDefaultValue(Function, *It, Defaults))
							{
								It->ImportText_Direct(*Defaults, It->ContainerPtrToValuePtr<uint8>(FuncParams->GetStructMemory()), nullptr, PPF_None);
							}

							// Check to see if the first parameter matches the selection object type, in that case we can directly forward the selection to it
							if (ParameterIndex == 0 && IsValidPropertyType(*It))
							{
								FirstParamProperty = *It;
							}

							++ParameterIndex;
						}

						bool bApply = true;

						if (!FirstParamProperty || ParameterIndex > 1)
						{
							// pop up a dialog to input params to the function
							TSharedRef<SWindow> Window = SNew(SWindow)
								.Title(Function->GetDisplayNameText())
								.ClientSize(FVector2D(400, 200))
								.SupportsMinimize(false)
								.SupportsMaximize(false);

							TSharedPtr<SFunctionParamDialog> Dialog;
							Window->SetContent(
								SAssignNew(Dialog, SFunctionParamDialog, Window, FuncParams, FirstParamProperty ? FirstParamProperty->GetFName() : NAME_None)
								.OkButtonText(LOCTEXT("OKButton", "OK"))
								.OkButtonTooltipText(Function->GetToolTipText()));

							GEditor->EditorAddModalWindow(Window);
							bApply = Dialog->bOKPressed;
						}


						if (bApply)
						{
							FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "BlutilityAction", "Blutility Action"));
							FEditorScriptExecutionGuard ScriptGuard;
							const bool bForwardUserSelection = FirstParamProperty != nullptr;
							if (bForwardUserSelection)
							{
								// For each user-select asset forward the selection object into the function first's parameter (if it matches)
								const FString Path = FirstParamProperty->GetPathName(Function);

								// Ensure we only process selection objects that are valid for this function/utility
								for (const int32& SelectionIndex : FunctionAndUtil.SelectionIndices)
								{
									const auto SelectedAsset = Selection[SelectionIndex];
									FirstParamProperty->CopySingleValue(FirstParamProperty->ContainerPtrToValuePtr<uint8>(FuncParams->GetStructMemory()), &SelectedAsset);
									TempObject->ProcessEvent(Function, FuncParams->GetStructMemory());
								}
							}
							else
							{
								// User is expected to manage the asset selection on its own
								TempObject->ProcessEvent(Function, FuncParams->GetStructMemory());
							}
						}
					}
					else
					{
						FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "BlutilityAction", "Blutility Action"));
						FEditorScriptExecutionGuard ScriptGuard;
						TempObject->ProcessEvent(Function, nullptr);
					}

					TempObject->RemoveFromRoot();
				}
			});

			FCanExecuteAction CanExecuteAction = FCanExecuteAction::CreateLambda([PassesFilterCondition]()
			{
				return PassesFilterCondition;
			});

			FToolMenuEntry MenuEntry = FToolMenuEntry::InitMenuEntry(
				FunctionAndUtil.FunctionData.Name,
				FunctionAndUtil.FunctionData.NameText,
				TooltipText,
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.Event_16x"),
				FUIAction(ExecuteAction, CanExecuteAction)
			);

			GeneratedMenuEntries.Emplace(MenuEntry);
		}

		return GeneratedMenuEntries;
	};

	// Add a menu item for each function
	if (CategoryFunctions.Num() > 0)
	{
		InSection.AddSubMenu(
			MenuName,
			MenuLabel,
			MenuToolTip,
			FNewToolMenuDelegate::CreateLambda([CategoryFunctions, AddFunctionEntries](UToolMenu* InMenu)
			{
				TArray<FString> CategoryNames;
				CategoryFunctions.GenerateKeyArray(CategoryNames);
				CategoryNames.Remove(FString());
				CategoryNames.Sort();

				FToolMenuSection& UnnamedSection = InMenu->FindOrAddSection(NAME_None);
				
				// Add functions belong to the same category to a sub-menu
				for (const FString& CategoryName : CategoryNames)
				{
					const TArray<FToolMenuEntry> GeneratedMenuEntries = AddFunctionEntries(CategoryFunctions.FindChecked(CategoryName));
					if (GeneratedMenuEntries.Num() > 0)
					{
						UnnamedSection.AddSubMenu(
							NAME_None,
							FText::FromString(CategoryName),
							FText::FromString(CategoryName),
							FNewToolMenuDelegate::CreateLambda([GeneratedMenuEntries](UToolMenu* InCategorySubMenu)
							{
								for (const FToolMenuEntry& MenuEntry : GeneratedMenuEntries)
								{
									InCategorySubMenu->AddMenuEntry(NAME_None, MenuEntry);
								}
							}
						));
					}
				}

				// Non-categorized functions
				const TArray<FFunctionAndUtil>* DefaultCategoryFunctionsPtr = CategoryFunctions.Find(FString());
				if (DefaultCategoryFunctionsPtr)
				{
					for (const FToolMenuEntry& MenuEntry : AddFunctionEntries(*DefaultCategoryFunctionsPtr))
					{
						InMenu->AddMenuEntry(NAME_None, MenuEntry);
					}
				}
			}),
			false,
			FSlateIcon(FAppStyle::GetAppStyleSetName(), IconName)
		);
	}
}

UEditorUtilityExtension::UEditorUtilityExtension(const class FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

#undef LOCTEXT_NAMESPACE 