// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "AssetToolsModule.h"
#include "AssetTypeCategories.h"
#include "BlutilityContentBrowserExtensions.h"
#include "BlutilityLevelEditorExtensions.h"
#include "BlutilityUMGEditorExtensions.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "EditorFunctionLibrary.h"
#include "EditorSupportDelegates.h"
#include "EditorUtilityActor.h"
#include "EditorUtilityBlueprint.h"
#include "EditorUtilityCamera.h"
#include "EditorUtilityCommon.h"
#include "EditorUtilityObject.h"
#include "EditorUtilitySubsystem.h"
#include "EditorUtilityWidget.h"
#include "EditorUtilityWidgetBlueprint.h"
#include "Engine/Blueprint.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/Docking/WorkspaceItem.h"
#include "GlobalEditorUtilityBase.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "IAssetTools.h"
#include "IBlutilityModule.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "KismetCompiler.h"
#include "KismetCompilerModule.h"
#include "LevelEditor.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/AssertionMacros.h"
#include "Modules/ModuleManager.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "Templates/SharedPointer.h"
#include "Templates/SubclassOf.h"
#include "Textures/SlateIcon.h"
#include "Trace/Detail/Channel.h"
#include "UMGEditorModule.h"
#include "UObject/Class.h"
#include "UObject/GCObject.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/Package.h"
#include "UObject/PurgingReferenceCollector.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/UObjectBase.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectHash.h"
#include "UnrealEdMisc.h"
#include "WidgetBlueprint.h"
#include "WidgetBlueprintCompiler.h"
#include "Widgets/Docking/SDockTab.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"
#include "EditorUtilityWidgetSettingsCustomization.h"
#include "EditorUtilityWidgetProjectSettings.h"
#include "PropertyEditorModule.h"

#define LOCTEXT_NAMESPACE "AssetTypeActions"

DEFINE_LOG_CATEGORY(LogEditorUtilityBlueprint);

/////////////////////////////////////////////////////

/////////////////////////////////////////////////////
// FBlutilityModule 

// Blutility module implementation (private)
class FBlutilityModule : public IBlutilityModule, public FGCObject
{
public:
	virtual void StartupModule() override
	{
		// Register the asset type
		IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
		EditorUtilityAssetCategory = AssetTools.RegisterAdvancedAssetCategory(FName(TEXT("Editor Utilities")), LOCTEXT("EditorUtilitiesAssetCategory", "Editor Utilities"));

		FKismetCompilerContext::RegisterCompilerForBP(UEditorUtilityWidgetBlueprint::StaticClass(), &UWidgetBlueprint::GetCompilerForWidgetBP);

		// Register widget blueprint compiler we do this no matter what - @todo: can i remove this? we're double registering..
		IUMGEditorModule& UMGEditorModule = FModuleManager::LoadModuleChecked<IUMGEditorModule>("UMGEditor");
		IKismetCompilerInterface& KismetCompilerModule = FModuleManager::LoadModuleChecked<IKismetCompilerInterface>("KismetCompiler");
		KismetCompilerModule.GetCompilers().Add(UMGEditorModule.GetRegisteredCompiler());
		KismetCompilerModule.OverrideBPTypeForClass(AEditorUtilityActor::StaticClass(), UEditorUtilityBlueprint::StaticClass());
		KismetCompilerModule.OverrideBPTypeForClass(AEditorUtilityCamera::StaticClass(), UEditorUtilityBlueprint::StaticClass());
		KismetCompilerModule.OverrideBPTypeForClass(UEditorUtilityObject::StaticClass(), UEditorUtilityBlueprint::StaticClass());
		KismetCompilerModule.OverrideBPTypeForClass(UEditorFunctionLibrary::StaticClass(), UEditorUtilityBlueprint::StaticClass());
		KismetCompilerModule.OverrideBPTypeForClass(UEditorUtilityWidget::StaticClass(), UEditorUtilityWidgetBlueprint::StaticClass());

		FBlutilityContentBrowserExtensions::InstallHooks();
		FBlutilityLevelEditorExtensions::InstallHooks();
		FBlutilityUMGEditorExtensions::InstallHooks();

		ScriptedEditorWidgetsGroup = WorkspaceMenu::GetMenuStructure().GetToolsCategory()->AddGroup(
			LOCTEXT("WorkspaceMenu_EditorUtilityWidgetsGroup", "Editor Utility Widgets"),
			LOCTEXT("ScriptedEditorWidgetsGroupTooltipText", "Custom editor UI created with Blueprints or Python."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "WorkspaceMenu.AdditionalUI"),
			true);

		FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
		LevelEditorModule.OnTabManagerChanged().AddRaw(this, &FBlutilityModule::ReinitializeUIs);
		LevelEditorModule.OnMapChanged().AddRaw(this, &FBlutilityModule::OnMapChanged);
		FEditorSupportDelegates::PrepareToCleanseEditorObject.AddRaw(this, &FBlutilityModule::OnPrepareToCleanseEditorObject);

		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		AssetRegistryModule.Get().OnAssetRemoved().AddRaw(this, &FBlutilityModule::HandleAssetRemoved);

		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.RegisterCustomClassLayout(UEditorUtilityWidgetProjectSettings::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FEditorUtilityWidgetSettingsCustomization::MakeInstance));
	}

	void ReinitializeUIs()
	{
		UEditorUtilitySubsystem* EditorUtilitySubsystem = GEditor->GetEditorSubsystem<UEditorUtilitySubsystem>();
		FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
		TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager();
		TArray<FSoftObjectPath> CorrectPaths;
		for (const FSoftObjectPath& BlueprintPath : EditorUtilitySubsystem->LoadedUIs)
		{
			UObject* BlueprintObject = BlueprintPath.TryLoad();
			if (BlueprintObject && IsValidChecked(BlueprintObject) && !BlueprintObject->IsUnreachable())
			{
				UEditorUtilityWidgetBlueprint* Blueprint = Cast<UEditorUtilityWidgetBlueprint>(BlueprintObject);
				if (Blueprint)
				{
					if (Blueprint->GeneratedClass)
					{
						const UEditorUtilityWidget* CDO = Blueprint->GeneratedClass->GetDefaultObject<UEditorUtilityWidget>();
						FName RegistrationName = FName(*(Blueprint->GetPathName() + LOCTEXT("ActiveTabSuffix", "_ActiveTab").ToString()));
						Blueprint->SetRegistrationName(RegistrationName);
						FText DisplayName = Blueprint->GetTabDisplayName();
						if (LevelEditorTabManager && !LevelEditorTabManager->HasTabSpawner(RegistrationName))
						{
							LevelEditorTabManager->RegisterTabSpawner(RegistrationName, FOnSpawnTab::CreateUObject(Blueprint, &UEditorUtilityWidgetBlueprint::SpawnEditorUITab))
								.SetDisplayName(DisplayName)
								.SetGroup(GetMenuGroup().ToSharedRef());
							CorrectPaths.Add(BlueprintPath);
						}
					}
					else
					{
						UE_LOGF(LogEditorUtilityBlueprint, Warning, "No generated class for: %ls", *BlueprintPath.ToString());
					}
				}
				else
				{
					UE_LOGF(LogEditorUtilityBlueprint, Warning, "Expected object of class EditorUtilityWidgetBlueprint: %ls", *BlueprintPath.ToString());
				}
			}
			else
			{
				UE_LOGF(LogEditorUtilityBlueprint, Warning, "Could not load: %ls", *BlueprintPath.ToString());
			}
		}

		EditorUtilitySubsystem->LoadedUIs = CorrectPaths;
		EditorUtilitySubsystem->SaveConfig();
	}

	void OnMapChanged(UWorld* InWorld, EMapChangeType MapChangeType)
	{
		for (const FSoftObjectPath& LoadedUI : GEditor->GetEditorSubsystem<UEditorUtilitySubsystem>()->LoadedUIs)
		{
			UEditorUtilityWidgetBlueprint* LoadedEditorUtilityBlueprint = Cast<UEditorUtilityWidgetBlueprint>(LoadedUI.ResolveObject());
			if (LoadedEditorUtilityBlueprint)
			{
				UEditorUtilityWidget* CreatedWidget = LoadedEditorUtilityBlueprint->GetCreatedWidget();
				if (CreatedWidget)
				{
					if (MapChangeType == EMapChangeType::TearDownWorld)
					{
						CreatedWidget->Rename(*CreatedWidget->GetName(), GetTransientPackage());
					}
					else if (MapChangeType == EMapChangeType::LoadMap || MapChangeType == EMapChangeType::NewMap)
					{
						UWorld* World = GEditor->GetEditorWorldContext().World();
						check(World);
						CreatedWidget->Rename(*CreatedWidget->GetName(), World);
					}
				}
			}
		}
	}

	virtual void ShutdownModule() override
	{
		if (!UObjectInitialized())
		{
			return;
		}

		// Unregister widget blueprint compiler we do this no matter what.
		IUMGEditorModule& UMGEditorModule = FModuleManager::LoadModuleChecked<IUMGEditorModule>("UMGEditor");
		IKismetCompilerInterface& KismetCompilerModule = FModuleManager::LoadModuleChecked<IKismetCompilerInterface>("KismetCompiler");
		KismetCompilerModule.GetCompilers().Remove(UMGEditorModule.GetRegisteredCompiler());

		FBlutilityLevelEditorExtensions::RemoveHooks();
		FBlutilityContentBrowserExtensions::RemoveHooks();
		FBlutilityUMGEditorExtensions::RemoveHooks();

		FEditorSupportDelegates::PrepareToCleanseEditorObject.RemoveAll(this);
	}

	virtual bool IsEditorUtilityBlueprint( const UBlueprint* Blueprint ) const override
	{
		const UClass* BPClass = Blueprint ? Blueprint->GetClass() : nullptr;
		const bool bIsExplicitEditorUtility = BPClass && 
			( BPClass->IsChildOf( UEditorUtilityBlueprint::StaticClass() )
				|| BPClass->IsChildOf( UEditorUtilityWidgetBlueprint::StaticClass() )
				|| IsEditorOnlyObject(BPClass) );

		if( bIsExplicitEditorUtility )
		{
			return true;
		}
		else
		{
			// People are frequently defining native editor only UObjects and 
			// using the editor only APIs. For consistency, indicate to callers
			// that these editor only classes should behave as editor utilities,
			// unless they are actors that want to run during pie:
			const UClass* NativeParent = FBlueprintEditorUtils::GetNativeParent(Blueprint);
			if(NativeParent && IsEditorOnlyObject(NativeParent))
			{
				if (NativeParent->IsChildOf(AActor::StaticClass()))
				{
					if(const AActor* NativeCDO = Cast<AActor>(NativeParent->GetDefaultObject(false)))
					{
						return !NativeCDO->IsEditorOnlyLoadedInPIE();
					}
				}
				return true;
			}
		}
		return false;
	}

	virtual TSharedPtr<class FWorkspaceItem> GetMenuGroup() const override
	{
		return ScriptedEditorWidgetsGroup;
	}

	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const TArray<FAssetCategoryPath, TFixedAllocator<1>> Categories = {
			FAssetCategoryPath(LOCTEXT("EditorUtilities", "Editor Utilities"))
		};
		return Categories;
	}

	virtual void AddLoadedScriptUI(class UEditorUtilityWidgetBlueprint* InBlueprint) override
	{
		UEditorUtilitySubsystem* EditorUtilitySubsystem = GEditor->GetEditorSubsystem<UEditorUtilitySubsystem>();
		EditorUtilitySubsystem->LoadedUIs.AddUnique(InBlueprint);
		EditorUtilitySubsystem->SaveConfig();
	}

	virtual void RemoveLoadedScriptUI(class UEditorUtilityWidgetBlueprint* InBlueprint) override
	{
		UEditorUtilitySubsystem* EditorUtilitySubsystem = GEditor->GetEditorSubsystem<UEditorUtilitySubsystem>();
		EditorUtilitySubsystem->LoadedUIs.Remove(InBlueprint);
		EditorUtilitySubsystem->SaveConfig();	
	}

protected:

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
	}
	virtual FString GetReferencerName() const override
	{
		return TEXT("FBlutilityModule");
	}

	void OnPrepareToCleanseEditorObject(UObject* InObject)
	{
		// Gather the list of live Editor Utility instances to purge references from
		TArray<UObject*> EditorUtilityInstances;
		ForEachObjectOfClass(UEditorUtilityWidget::StaticClass(), [&EditorUtilityInstances](UObject* InEditorUtilityInstance)
		{
			EditorUtilityInstances.Add(InEditorUtilityInstance);
		});
		ForEachObjectOfClass(UDEPRECATED_GlobalEditorUtilityBase::StaticClass(), [&EditorUtilityInstances](UObject* InEditorUtilityInstance)
		{
			EditorUtilityInstances.Add(InEditorUtilityInstance);
		});

		if (EditorUtilityInstances.Num() > 0)
		{
			// Build up the complete list of objects to purge
			FPurgingReferenceCollector PurgingReferenceCollector;
			PurgingReferenceCollector.AddObjectToPurge(InObject);
			ForEachObjectWithOuter(InObject, [&PurgingReferenceCollector](UObject* InInnerObject)
			{
				PurgingReferenceCollector.AddObjectToPurge(InInnerObject);
			}, EGetObjectsFlags::IncludeNestedObjects);

			// Run the purge for each Editor Utility instance
			FReferenceCollectorArchive& PurgingReferenceCollectorArchive = PurgingReferenceCollector.GetVerySlowReferenceCollectorArchive();
			for (UObject* EditorUtilityInstance : EditorUtilityInstances)
			{
				PurgingReferenceCollectorArchive.SetSerializingObject(EditorUtilityInstance);
				EditorUtilityInstance->Serialize(PurgingReferenceCollectorArchive);
				EditorUtilityInstance->CallAddReferencedObjects(PurgingReferenceCollector);
				PurgingReferenceCollectorArchive.SetSerializingObject(nullptr);
			}
		}
	}

	void HandleAssetRemoved(const FAssetData& InAssetData)
	{
		bool bDeletingLoadedUI = false;
		if (!GEditor || !GEditor->GetEditorSubsystem<UEditorUtilitySubsystem>())
		{
			return;
		}
		for (const FSoftObjectPath& LoadedUIPath : GEditor->GetEditorSubsystem<UEditorUtilitySubsystem>()->LoadedUIs)
		{
			if (LoadedUIPath == InAssetData.GetSoftObjectPath())
			{
				bDeletingLoadedUI = true;
				break;
			}
		}

		if (bDeletingLoadedUI)
		{
			FName UIToCleanup = FName(*(InAssetData.GetObjectPathString() + LOCTEXT("ActiveTabSuffix", "_ActiveTab").ToString()));
			FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
			TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager();
			TSharedPtr<SDockTab> CurrentTab = LevelEditorTabManager->FindExistingLiveTab(UIToCleanup);
			if (CurrentTab.IsValid())
			{
				CurrentTab->RequestCloseTab();
			}
		}
	}
protected:
	/** Scripted Editor Widgets workspace menu item */
	TSharedPtr<class FWorkspaceItem> ScriptedEditorWidgetsGroup;

	EAssetTypeCategories::Type EditorUtilityAssetCategory;
};



IMPLEMENT_MODULE( FBlutilityModule, Blutility );
#undef LOCTEXT_NAMESPACE 
