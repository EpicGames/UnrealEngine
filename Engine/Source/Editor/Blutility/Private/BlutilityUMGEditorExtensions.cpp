// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlutilityUMGEditorExtensions.h"

#include "IDocumentation.h"
#include "BlueprintEditor.h"
#include "BlueprintEditorContext.h"
#include "Blueprint/UserWidget.h"
#include "BlueprintModes/WidgetBlueprintApplicationModes.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "EditorUtilitySubsystem.h"
#include "EditorUtilityWidget.h"
#include "EditorUtilityWidgetBlueprint.h"
#include "Modules/ModuleManager.h"
#include "ToolMenus.h"
#include "UMGEditorModule.h"
#include "WidgetBlueprint.h"
#include "WidgetBlueprintEditor.h"

class FUICommandList;

#define LOCTEXT_NAMESPACE "BlutilityUMGEditorExtensions"

namespace UE::Blutility::Private
{
	// Hardcoded toolbar name since not exposed without instance of UMG editor
	static FName WidgetBlueprintEditorToolbarName = FName("AssetEditor.WidgetBlueprintEditor.ToolBar");
	static FName BlutilityExtensionName = FName("BlutilityExtensions");
}

class FBlutilityUMGEditorExtensions_Impl
{
public:

	static void AddEditorUtilityExtension(UToolMenu* InMenu)
	{
		FToolMenuSection& Section = InMenu->AddSection(UE::Blutility::Private::BlutilityExtensionName);
		Section.InsertPosition = FToolMenuInsert("Compile", EToolMenuInsertType::After);
		Section.AddDynamicEntry("RunStopUtilityWidget", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
		{
			UBlueprintEditorToolMenuContext* Context = InSection.FindContext<UBlueprintEditorToolMenuContext>();
			if (Context)
			{
				if (TSharedPtr<FWidgetBlueprintEditor> WidgetBlueprintEditor = StaticCastSharedPtr<FWidgetBlueprintEditor>(Context->BlueprintEditor.Pin()))
				{
					if (WidgetBlueprintEditor->GetPreview() && WidgetBlueprintEditor->GetPreview()->IsEditorUtility())
					{
						TWeakPtr<FWidgetBlueprintEditor> WeakWidgetBlueprintEditor = WidgetBlueprintEditor;

						FName UtilityWidgetName = WidgetBlueprintEditor->GetBlueprintObj()->GetFName();

						// Add Run Button, will toggle if tab already ran.
						InSection.AddEntry(FToolMenuEntry::InitToolBarButton(
							"RunUtilityWidget",
							FUIAction(
								FExecuteAction::CreateLambda([WeakWidgetBlueprintEditor]()
									{ 
										if (TSharedPtr<FWidgetBlueprintEditor> WidgetBlueprintEditor = WeakWidgetBlueprintEditor.Pin())
										{
											if (UEditorUtilityWidgetBlueprint* EditorWidget = Cast<UEditorUtilityWidgetBlueprint>(WidgetBlueprintEditor->GetBlueprintObj()))
											{
												GEditor->GetEditorSubsystem<UEditorUtilitySubsystem>()->SpawnAndRegisterTabWithId(EditorWidget, FName("_ActiveTab"));
											}
										}
									}),
								FCanExecuteAction(),
								FGetActionCheckState(),
								FIsActionButtonVisible::CreateLambda([WeakWidgetBlueprintEditor]()
									{
										if (TSharedPtr<FWidgetBlueprintEditor> WidgetBlueprintEditor = WeakWidgetBlueprintEditor.Pin())
										{
											if (UEditorUtilityWidgetBlueprint* EditorWidget = Cast<UEditorUtilityWidgetBlueprint>(WidgetBlueprintEditor->GetBlueprintObj()))
											{
												// Editor Utility Prepends path on Tab Name
												FName UtilityTabName = FName(*(EditorWidget->GetPathName() + FString("_ActiveTab")));
												return !GEditor->GetEditorSubsystem<UEditorUtilitySubsystem>()->DoesTabExist(UtilityTabName);
											}
										}
										return false; 
									})
							)
							, LOCTEXT("RunUtilityWidget", "Run Utility Widget")
							, LOCTEXT("RunUtilityWidgetToolTip", "Run Utility Widget, tab name will be based off of asset.")
							, FSlateIcon(FAppStyle::Get().GetStyleSetName(), "PlayWorld.PlayInViewport") // @TODO: DarenC - Placeholder
						));

						// Add Stop Button, will toggle if tab already ran.
						InSection.AddEntry(FToolMenuEntry::InitToolBarButton(
							"StopUtilityWidget",
							FUIAction(
								FExecuteAction::CreateLambda([WeakWidgetBlueprintEditor]()
									{
										if (TSharedPtr<FWidgetBlueprintEditor> WidgetBlueprintEditor = WeakWidgetBlueprintEditor.Pin())
										{
											if (UEditorUtilityWidgetBlueprint* EditorWidget = Cast<UEditorUtilityWidgetBlueprint>(WidgetBlueprintEditor->GetBlueprintObj()))
											{
												// Editor Utility Prepends path on Tab Name
												FName UtilityTabName = FName(*(EditorWidget->GetPathName() + FString("_ActiveTab")));
												GEditor->GetEditorSubsystem<UEditorUtilitySubsystem>()->CloseTabByID(UtilityTabName);
											}
										}
									}),
								FCanExecuteAction(),
								FGetActionCheckState(),
								FIsActionButtonVisible::CreateLambda([WeakWidgetBlueprintEditor]()
									{
										if (TSharedPtr<FWidgetBlueprintEditor> WidgetBlueprintEditor = WeakWidgetBlueprintEditor.Pin())
										{
											if (UEditorUtilityWidgetBlueprint* EditorWidget = Cast<UEditorUtilityWidgetBlueprint>(WidgetBlueprintEditor->GetBlueprintObj()))
											{
												// Editor Utility Prepends path on Tab Name
												FName UtilityTabName = FName(*(EditorWidget->GetPathName() + FString("_ActiveTab")));
												return GEditor->GetEditorSubsystem<UEditorUtilitySubsystem>()->DoesTabExist(UtilityTabName);
											}
										}
										return false; 
									})
							)
							, LOCTEXT("StopUtilityWidget", "Stop Utility Widget")
							, LOCTEXT("StopUtilityWidgetToolTip", "Stop Utility Widget, will stop tab based off of asset name (Does not stop manually ran instances).")
							, FSlateIcon(FAppStyle::Get().GetStyleSetName(), "PlayWorld.StopPlaySession") // @TODO: DarenC - Placeholder
						));
					}
				}
			}
		}));
	}

	static void RemoveEditorUtilityExtension(UToolMenu* InMenu)
	{
		InMenu->RemoveSection(UE::Blutility::Private::BlutilityExtensionName);
	}
};

void FBlutilityUMGEditorExtensions::InstallHooks()
{
	// Load module to ensure relevant toolbar exists
	IUMGEditorModule& UMGEditorModule = FModuleManager::Get().LoadModuleChecked<IUMGEditorModule>("UMGEditor");

	auto AddExtensionsUMGEditorToolbar = [](const FName InModeName, FName ParentToolbarName)
	{
		const FName ModeSpecificToolbarName = *(ParentToolbarName.ToString() + TEXT(".") + InModeName.ToString());

		if (!UToolMenus::Get()->IsMenuRegistered(ModeSpecificToolbarName))
		{
			UToolMenus::Get()->RegisterMenu(ModeSpecificToolbarName, ParentToolbarName, EMultiBoxType::ToolBar);
		}

		if (UToolMenu* Toolbar = UToolMenus::Get()->FindMenu(ModeSpecificToolbarName))
		{
			FBlutilityUMGEditorExtensions_Impl::AddEditorUtilityExtension(Toolbar);
		}
	};

	AddExtensionsUMGEditorToolbar(FWidgetBlueprintApplicationModes::DesignerMode, UE::Blutility::Private::WidgetBlueprintEditorToolbarName);
	AddExtensionsUMGEditorToolbar(FWidgetBlueprintApplicationModes::GraphMode, UE::Blutility::Private::WidgetBlueprintEditorToolbarName);
}

void FBlutilityUMGEditorExtensions::RemoveHooks()
{
	// Load module to ensure relevant toolbar exists
	IUMGEditorModule& UMGEditorModule = FModuleManager::Get().LoadModuleChecked<IUMGEditorModule>("UMGEditor");

	auto RemoveExtensionsUMGEditorToolbar = [](const FName InModeName, FName ParentToolbarName)
	{
		const FName ModeSpecificToolbarName = *(ParentToolbarName.ToString() + TEXT(".") + InModeName.ToString());

		if (!UToolMenus::Get()->IsMenuRegistered(ModeSpecificToolbarName))
		{
			UToolMenus::Get()->RegisterMenu(ModeSpecificToolbarName, ParentToolbarName, EMultiBoxType::ToolBar);
		}

		if (UToolMenu* Toolbar = UToolMenus::Get()->FindMenu(ModeSpecificToolbarName))
		{
			FBlutilityUMGEditorExtensions_Impl::RemoveEditorUtilityExtension(Toolbar);
		}
	};

	RemoveExtensionsUMGEditorToolbar(FWidgetBlueprintApplicationModes::DesignerMode, UE::Blutility::Private::WidgetBlueprintEditorToolbarName);
	RemoveExtensionsUMGEditorToolbar(FWidgetBlueprintApplicationModes::GraphMode, UE::Blutility::Private::WidgetBlueprintEditorToolbarName);
}

#undef LOCTEXT_NAMESPACE
