// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Integrate Blutility actions associated with UMG editor functionality (e.g. Run / Stop Utility Widget)
class FBlutilityUMGEditorExtensions
{
public:
	static void InstallHooks();
	static void RemoveHooks();
};
