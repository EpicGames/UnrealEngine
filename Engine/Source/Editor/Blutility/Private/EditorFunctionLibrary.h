// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "EditorFunctionLibrary.generated.h"

/**
 * Library of static functions that can use the editor APIs
 */
UCLASS(Abstract, MinimalAPI)
class UEditorFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
};

/*
	Note: There is no reason to extend this in c++, it is used as a sentinel type for blueprints
	that want to provide static reusable functions for editing logic. In c++ you can just extend
	UBlueprintFunctionLibrary as normal and put your class in an uncooked or editor only module
	or WITH_EDITOR block.
*/
