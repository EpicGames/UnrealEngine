// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorUtilityActor.h"
#include "GameFramework/InputSettings.h"
#include "Engine/InputDelegateBinding.h"

/////////////////////////////////////////////////////

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorUtilityActor)

AEditorUtilityActor::AEditorUtilityActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void AEditorUtilityActor::OnConstruction(const FTransform& Transform)
{
	Super::OnConstruction(Transform);

	CreateEditorInput();
}

void AEditorUtilityActor::CreateEditorInput()
{
	if (bReceivesEditorInput && !HasAnyFlags(RF_ClassDefaultObject) && !EditorOnlyInputComponent)
	{
		EditorOnlyInputComponent = NewObject<UInputComponent>(this, UInputSettings::GetDefaultInputComponentClass(), TEXT("EdUtilActor_InputComponent0"), RF_Transient);
		UInputDelegateBinding::BindInputDelegatesWithSubojects(this, EditorOnlyInputComponent);
	}
}

void AEditorUtilityActor::RemoveEditorInput()
{
	ensure(!bReceivesEditorInput);
	
	if (EditorOnlyInputComponent)
	{
		EditorOnlyInputComponent->DestroyComponent();
	}
	EditorOnlyInputComponent = nullptr;
}

#if WITH_EDITOR
void AEditorUtilityActor::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = (PropertyChangedEvent.Property != nullptr) ? PropertyChangedEvent.Property->GetFName() : NAME_None;
	if (PropertyName == GET_MEMBER_NAME_CHECKED(AEditorUtilityActor, bReceivesEditorInput))
	{
		SetReceivesEditorInput(bReceivesEditorInput);
	}
}
#endif


void AEditorUtilityActor::SetReceivesEditorInput(bool bInValue)
{
	bReceivesEditorInput = bInValue;
	if (bReceivesEditorInput)
	{
		CreateEditorInput();
	}
	else
	{
		RemoveEditorInput();
	}
}
