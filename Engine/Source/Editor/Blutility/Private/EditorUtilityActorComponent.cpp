// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorUtilityActorComponent.h"


/////////////////////////////////////////////////////

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorUtilityActorComponent)

UEditorUtilityActorComponent::UEditorUtilityActorComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}
