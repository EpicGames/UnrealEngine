// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorUtilityAssetPrototype.h"

#include "AssetActionUtility.h"
#include "EditorUtilityBlueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "JsonObjectConverter.h"
#include "AssetRegistry/AssetData.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "HAL/IConsoleManager.h"
#include "Serialization/JsonSerializerMacros.h"
#include "UObject/AssetRegistryTagsContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorUtilityAssetPrototype)

namespace AssetActionUtilityConsoleVariables
{
	bool bHideAmbiguousScriptedAssetActions = true;
	FAutoConsoleVariableRef CVarEnableMutatorsSubsystem(
		TEXT("AssetActionUtility.HideAmbiguousScriptedAssetActions"),
		bHideAmbiguousScriptedAssetActions,
		TEXT("If enabled, Scripted Asset Actions will only show up if they explicitly define Supported Class(es) via their Class Defaults.\n0: Disable, 1: Enable (default)"),
		ECVF_Default);

} // namespace AssetActionUtilityConsoleVariables

const FName AssetActionUtilityTags::BlutilityTagVersion(TEXT("BlutilityTagVersion"));
const FName AssetActionUtilityTags::SupportedClasses(TEXT("SupportedClasses"));
const FName AssetActionUtilityTags::IsActionForBlueprint(TEXT("IsActionForBlueprint"));
const FName AssetActionUtilityTags::CallableFunctions(TEXT("CallableFunctions"));
const FName AssetActionUtilityTags::SupportedConditions(TEXT("SupportedConditions"));

const int32 AssetActionUtilityTags::TagVersion = 1;

UObject* FAssetActionUtilityPrototype::LoadUtilityAsset() const
{
	if (const UEditorUtilityBlueprint* Blueprint = Cast<UEditorUtilityBlueprint>(UtilityBlueprintAsset.GetAsset()))
	{
		if (const UClass* BPClass = Blueprint->GeneratedClass.Get())
		{
			return BPClass->GetDefaultObject();
		}
	}
	else if (const UBlueprintGeneratedClass* BPClass = Cast<UBlueprintGeneratedClass>(UtilityBlueprintAsset.GetAsset()))
	{
		return BPClass->GetDefaultObject();
	}

	return nullptr;
};

bool FAssetActionUtilityPrototype::IsLatestVersion() const
{
	int32 Version;
    if (UtilityBlueprintAsset.GetTagValue(AssetActionUtilityTags::BlutilityTagVersion, Version))
    {
    	return Version == AssetActionUtilityTags::TagVersion;
    }
    
    return false;
}

bool FAssetActionUtilityPrototype::AreSupportedClassesForBlueprints() const
{
	FString IsActionForBlueprintString;
	if (UtilityBlueprintAsset.GetTagValue(AssetActionUtilityTags::IsActionForBlueprint, IsActionForBlueprintString))
	{
		bool IsActionForBlueprint = false;
		LexFromString(IsActionForBlueprint, *IsActionForBlueprintString);
		return IsActionForBlueprint;
	}

	return false;
}

TArray<TSoftClassPtr<UObject>> FAssetActionUtilityPrototype::GetSupportedClasses() const
{
    TArray<TSoftClassPtr<UObject>> SupportedClasses;

	FString SupportedClassPaths;
	if (UtilityBlueprintAsset.GetTagValue(AssetActionUtilityTags::SupportedClasses, SupportedClassPaths))
	{
		TArray<FString> ClassPaths;
		SupportedClassPaths.ParseIntoArray(ClassPaths, TEXT(","));
		Algo::Transform(ClassPaths, SupportedClasses, [](const FString& ClassPath) { return TSoftClassPtr<UObject>(FSoftClassPath(ClassPath)); });
	}

	if (SupportedClasses.Num() == 0)
	{
		// If this cvar is set, we DON'T assume an empty array means we support UObject and derived classes
		// This cleans up the "Scripted Asset Actions" list and ensures only actions that explicitly support the asset class being requested show up
		if (!AssetActionUtilityConsoleVariables::bHideAmbiguousScriptedAssetActions)
		{
			SupportedClasses.Add(TSoftClassPtr<UObject>(UObject::StaticClass()));
		}
	}
	
	return SupportedClasses;
}

TArray<FBlutilityFunctionData> FAssetActionUtilityPrototype::GetCallableFunctions() const
{
	TArray<FBlutilityFunctionData> FunctionDatas;
	
	FString FunctionDataJson;
	if (UtilityBlueprintAsset.GetTagValue(AssetActionUtilityTags::CallableFunctions, FunctionDataJson))
	{
		FJsonObjectConverter::JsonArrayStringToUStruct(FunctionDataJson, &FunctionDatas, 0, 0);
	}

	return FunctionDatas;
}

TArray<FAssetActionSupportCondition> FAssetActionUtilityPrototype::GetAssetActionSupportConditions() const
{
	TArray<FAssetActionSupportCondition> Results;
	
	FString Json;
	if (UtilityBlueprintAsset.GetTagValue(AssetActionUtilityTags::SupportedConditions, Json))
	{
		FJsonObjectConverter::JsonArrayStringToUStruct(Json, &Results, 0, 0);
	}

	return Results;
}

void FAssetActionUtilityPrototype::AddTagsFor_Version(FAssetRegistryTagsContext Context)
{
	// Adding a version to the tags just in case we need to know 'this blutility is out of date and we can't go based off the cached data because the format isn't followed, or this is a pre-tagged version'
	Context.AddTag(UObject::FAssetRegistryTag(AssetActionUtilityTags::BlutilityTagVersion, LexToString(AssetActionUtilityTags::TagVersion), UObject::FAssetRegistryTag::TT_Hidden));	
}

void FAssetActionUtilityPrototype::AddTagsFor_SupportedClasses(const TArray<TSoftClassPtr<UObject>>& SupportedClasses, FAssetRegistryTagsContext Context)
{
	const FString SupportedClassesString = FString::JoinBy(SupportedClasses, TEXT(","), [](const TSoftClassPtr<UObject>& SupportedClass) { return SupportedClass.ToString(); });
	Context.AddTag(UObject::FAssetRegistryTag(AssetActionUtilityTags::SupportedClasses, SupportedClassesString, UObject::FAssetRegistryTag::TT_Hidden));
}

void FAssetActionUtilityPrototype::AddTagsFor_SupportedConditions(const TArray<FAssetActionSupportCondition>& SupportedConditions, FAssetRegistryTagsContext Context)
{
	TArray<TSharedPtr<FJsonValue>> SupportedConditionJsonValues;
	for (const FAssetActionSupportCondition& Condition : SupportedConditions)
	{
		TSharedPtr<FJsonObject> ConditionJO = FJsonObjectConverter::UStructToJsonObject(Condition);
		if (ConditionJO.IsValid())
		{
			SupportedConditionJsonValues.Add(MakeShared<FJsonValueObject>(ConditionJO));
		}
	}

	if (SupportedConditionJsonValues.Num() > 0)
	{
		FString SupportedConditionsJson;
		{
			TSharedRef<FJsonValueArray> JsonArray = MakeShared<FJsonValueArray>(SupportedConditionJsonValues);
			TSharedRef<TJsonWriter<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>> JsonWriter = TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>::Create(&SupportedConditionsJson);
			FJsonSerializer::Serialize(JsonArray, FString(), JsonWriter);
		}

		Context.AddTag(UObject::FAssetRegistryTag(AssetActionUtilityTags::SupportedConditions, SupportedConditionsJson, UObject::FAssetRegistryTag::TT_Hidden));
	}
}

void FAssetActionUtilityPrototype::AddTagsFor_IsActionForBlueprints(bool IsActionForBlueprints, FAssetRegistryTagsContext Context)
{
	const bool ActionForBlueprint = IsActionForBlueprints;
	Context.AddTag(UObject::FAssetRegistryTag(AssetActionUtilityTags::IsActionForBlueprint, ActionForBlueprint ? TEXT("True") : TEXT("False"), UObject::FAssetRegistryTag::TT_Hidden));
}

void FAssetActionUtilityPrototype::AddTagsFor_CallableFunctions(const UObject* FunctionsSource, FAssetRegistryTagsContext Context)
{
	check(FunctionsSource);
	
	const static FName NAME_CallInEditor(TEXT("CallInEditor"));
	const static FName NAME_Category(TEXT("Category"));

	TArray<FBlutilityFunctionData> FunctionDatas;
	for (TFieldIterator<UFunction> FunctionIt(FunctionsSource->GetClass(), EFieldIterationFlags::IncludeSuper); FunctionIt; ++FunctionIt)
	{
		if (UFunction* Func = *FunctionIt)
		{
			if (Func->HasMetaData(NAME_CallInEditor) && Func->GetReturnProperty() == nullptr)
			{
				FBlutilityFunctionData Function;
				Function.Class = Func->GetOuterUClass();
				Function.Name = Func->GetFName();
				Function.NameText = Func->GetDisplayNameText();
				Function.Category = Func->GetMetaData(NAME_Category);
				Function.TooltipText = Func->GetToolTipText();
				
				FunctionDatas.Add(MoveTemp(Function));
			}
		}
	}

	TArray<TSharedPtr<FJsonValue>> FunctionDataJsonValues;
	for (const FBlutilityFunctionData& FunctionData : FunctionDatas)
	{
		TSharedPtr<FJsonObject> FunctionDataJO = FJsonObjectConverter::UStructToJsonObject(FunctionData);
		if (FunctionDataJO.IsValid())
		{
			FunctionDataJsonValues.Add(MakeShared<FJsonValueObject>(FunctionDataJO));
		}
	}

	if (FunctionDataJsonValues.Num() > 0)
	{
		FString FunctionDataJson;
		{
			TSharedRef<FJsonValueArray> JsonArray = MakeShared<FJsonValueArray>(FunctionDataJsonValues);
			TSharedRef<TJsonWriter<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>> JsonWriter = TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>::Create(&FunctionDataJson);
			FJsonSerializer::Serialize(JsonArray, FString(), JsonWriter);
		}

		Context.AddTag(UObject::FAssetRegistryTag(AssetActionUtilityTags::CallableFunctions, FunctionDataJson, UObject::FAssetRegistryTag::TT_Hidden));
	}
}
