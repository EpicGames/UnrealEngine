// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorUtilityBlueprint.h"

/////////////////////////////////////////////////////
// UEditorUtilityBlueprint

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorUtilityBlueprint)

UEditorUtilityBlueprint::UEditorUtilityBlueprint(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

bool UEditorUtilityBlueprint::SupportedByDefaultBlueprintFactory() const
{
	return false;
}

bool UEditorUtilityBlueprint::AlwaysCompileOnLoad() const
{
	return true;
}
