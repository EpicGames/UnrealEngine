// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorUtilityBlueprintFactory.h"

#include "AssetActionUtility.h"
#include "ClassViewerFilter.h"
#include "ClassViewerModule.h"
#include "Containers/Array.h"
#include "Containers/Set.h"
#include "EditorUtilityActor.h"
#include "EditorUtilityBlueprint.h"
#include "EditorUtilityObject.h"
#include "EditorUtilityWidget.h"
#include "EditorFunctionLibrary.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMisc.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Kismet2/SClassPickerDialog.h"
#include "Misc/AssertionMacros.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "UObject/Class.h"
#include "Widgets/SWindow.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorUtilityBlueprintFactory)

class FFeedbackContext;
class UObject;

class FBlutilityBlueprintFactoryFilter : public IClassViewerFilter
{
public:
	TSet< const UClass* > AllowedChildOfClasses;

	TSet< const UClass*> DisallowedChildOfClasses;

	bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs ) override
	{
		if (DisallowedChildOfClasses.Num() == 0 && AllowedChildOfClasses.Num() == 0)
		{
			return true;
		}
		return (InFilterFuncs->IfInChildOfClassesSet(AllowedChildOfClasses, InClass) != EFilterReturn::Failed) 
			&& (InFilterFuncs->IfInChildOfClassesSet(DisallowedChildOfClasses, InClass) == EFilterReturn::Failed);
	}

	virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const IUnloadedBlueprintData > InUnloadedClassData, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
	{
		if (DisallowedChildOfClasses.Num() == 0 && AllowedChildOfClasses.Num() == 0)
		{
			return true;
		}

		return InFilterFuncs->IfInChildOfClassesSet(AllowedChildOfClasses, InUnloadedClassData) != EFilterReturn::Failed
			&& (InFilterFuncs->IfInChildOfClassesSet(DisallowedChildOfClasses, InUnloadedClassData) == EFilterReturn::Failed);;
	}
};

/////////////////////////////////////////////////////
// UEditorUtilityBlueprintFactory

UEditorUtilityBlueprintFactory::UEditorUtilityBlueprintFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UEditorUtilityBlueprint::StaticClass();
}

bool UEditorUtilityBlueprintFactory::ConfigureProperties()
{
	// Null the parent class so we can check for selection later
	ParentClass = NULL;

	// Load the class viewer module to display a class picker
	FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");

	// Fill in options
	FClassViewerInitializationOptions Options;
	Options.Mode = EClassViewerMode::ClassPicker;
	Options.DisplayMode = EClassViewerDisplayMode::TreeView;
	// Only want blueprint actor base classes.
	Options.bIsBlueprintBaseOnly = true;
	// This will allow unloaded blueprints to be shown.
	Options.bShowUnloadedBlueprints = true;
	Options.bEditorClassesOnly = true;

	Options.ExtraPickerCommonClasses.Add(AEditorUtilityActor::StaticClass());
	Options.ExtraPickerCommonClasses.Add(UEditorUtilityObject::StaticClass());
	Options.ExtraPickerCommonClasses.Add(UAssetActionUtility::StaticClass());
	Options.ExtraPickerCommonClasses.Add(UEditorFunctionLibrary::StaticClass());

	TSharedPtr< FBlutilityBlueprintFactoryFilter > Filter = MakeShareable(new FBlutilityBlueprintFactoryFilter);
	Filter->DisallowedChildOfClasses.Add(UEditorUtilityWidget::StaticClass());
	Options.ClassFilters.Add(Filter.ToSharedRef());

	const FText TitleText = NSLOCTEXT("EditorFactories", "CreateBlueprintOptions", "Pick Parent Class");
	UClass* ChosenClass = NULL;
	const bool bPressedOk = SClassPickerDialog::PickClass(TitleText, Options, ChosenClass, UEditorUtilityBlueprint::StaticClass());
	if (bPressedOk)
	{
		ParentClass = ChosenClass;
	}

	return bPressedOk;
}

UObject* UEditorUtilityBlueprintFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	// Make sure we are trying to factory a blueprint, then create and init one
	check(Class->IsChildOf(UBlueprint::StaticClass()));

	EBlueprintType BPType = BPTYPE_Normal;
	if (ParentClass == UEditorFunctionLibrary::StaticClass())
	{
		BPType = BPTYPE_FunctionLibrary;
	}
	else if ((ParentClass == NULL) || !FKismetEditorUtilities::CanCreateBlueprintOfClass(ParentClass))
	{
		FFormatNamedArguments Args;
		Args.Add( TEXT("ClassName"), (ParentClass != NULL) ? FText::FromString( ParentClass->GetName() ) : NSLOCTEXT("UnrealEd", "Null", "(null)") );
		FMessageDialog::Open( EAppMsgType::Ok, FText::Format( NSLOCTEXT("UnrealEd", "CannotCreateBlueprintFromClass", "Cannot create a blueprint based on the class '{ClassName}'."), Args ) );
		return NULL;
	}

	return FKismetEditorUtilities::CreateBlueprint(ParentClass, InParent, Name, BPType, UEditorUtilityBlueprint::StaticClass(), UBlueprintGeneratedClass::StaticClass());
}

bool UEditorUtilityBlueprintFactory::CanCreateNew() const
{
	return true;
}

void UEditorUtilityBlueprintFactory::OnClassPicked(UClass* InChosenClass)
{
	ParentClass = InChosenClass;
	PickerWindow.Pin()->RequestDestroyWindow();
}
