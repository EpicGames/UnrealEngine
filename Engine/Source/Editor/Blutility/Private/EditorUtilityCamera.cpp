// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorUtilityCamera.h"


/////////////////////////////////////////////////////

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorUtilityCamera)

AEditorUtilityCamera::AEditorUtilityCamera(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}
