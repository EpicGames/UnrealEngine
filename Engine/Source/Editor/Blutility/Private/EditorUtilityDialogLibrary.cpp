// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorUtilityDialogLibrary.h"
#include "SEditorUtilityDialog.h"
#include "EditorUtilityDialogWidget.h"

#include "EditorUtilityCommon.h"
#include "Editor.h"
#include "Blueprint/UserWidget.h"

FEditorUtilityDialogResult UEditorUtilityDialogLibrary::ShowModalEditorUtilityDialog(
	TSubclassOf<UEditorUtilityDialogWidget> WidgetClass,
	FText Title,
	EAppMsgType::Type ButtonLayout,
	FVector2D Size)
{
	FEditorUtilityDialogResult Result;

	TPair<UEditorUtilityDialogWidget*, TSharedPtr<SEditorUtilityDialog>> Created = CreateDialogInternal(WidgetClass, Title, ButtonLayout, Size);
	UEditorUtilityDialogWidget* Widget = Created.Key;
	TSharedPtr<SEditorUtilityDialog> Dialog = Created.Value;
	if (!Widget || !Dialog.IsValid())
	{
		return Result;
	}

	Widget->OnDialogOpened();

	// Show modal - blocks until closed. SEditorUtilityDialog::ShowModal handles
	// unattended mode internally (fires widget close events and returns Cancel).
	Result.ReturnType = Dialog->ShowModal();

	// Read the hosted widget AFTER ShowModal returns - a Blueprint recompile during
	// the dialog would have swapped the instance, and callers need the rebuilt one.
	Result.WidgetInstance = Dialog->GetHostedWidget();

	return Result;
}

UEditorUtilityDialogWidget* UEditorUtilityDialogLibrary::ShowEditorUtilityDialog(
	TSubclassOf<UEditorUtilityDialogWidget> WidgetClass,
	const FOnEditorUtilityDialogClosed& OnClosed,
	FText Title,
	EAppMsgType::Type ButtonLayout,
	FVector2D Size)
{
	TPair<UEditorUtilityDialogWidget*, TSharedPtr<SEditorUtilityDialog>> Created = CreateDialogInternal(WidgetClass, Title, ButtonLayout, Size);
	UEditorUtilityDialogWidget* Widget = Created.Key;
	TSharedPtr<SEditorUtilityDialog> Dialog = Created.Value;
	if (!Widget || !Dialog.IsValid())
	{
		return nullptr;
	}

	Widget->OnDialogOpened();
	BindNonModalCloseCallback(*Dialog, Widget, OnClosed);

	return Widget;
}

UEditorUtilityDialogWidget* UEditorUtilityDialogLibrary::CreateEditorUtilityDialogWidget(
	TSubclassOf<UEditorUtilityDialogWidget> WidgetClass)
{
	// Delegates to SEditorUtilityDialog::CreateHostedWidget -- single source of truth for
	// world resolution, CreateWidget, RF_Transient flagging, and error logging.
	// CreateWidget calls Initialize() which builds the widget tree, so child widget
	// references (text blocks, etc.) are valid after this.
	return SEditorUtilityDialog::CreateHostedWidget(WidgetClass);
}

FEditorUtilityDialogResult UEditorUtilityDialogLibrary::ShowModalEditorUtilityDialogInstance(
	UEditorUtilityDialogWidget* WidgetInstance,
	FText Title,
	EAppMsgType::Type ButtonLayout,
	FVector2D Size)
{
	FEditorUtilityDialogResult Result;

	if (!WidgetInstance)
	{
		UE_LOGF(LogEditorUtilityBlueprint, Error, "ShowModalEditorUtilityDialogInstance: WidgetInstance is null.");
		return Result;
	}

	TSharedPtr<SEditorUtilityDialog> Dialog = WrapWidgetInDialog(WidgetInstance, Title, ButtonLayout, Size);
	if (!Dialog.IsValid())
	{
		return Result;
	}

	WidgetInstance->OnDialogOpened();

	Result.ReturnType = Dialog->ShowModal();

	// Read the hosted widget AFTER ShowModal returns in case a recompile swapped it.
	Result.WidgetInstance = Dialog->GetHostedWidget();

	return Result;
}

void UEditorUtilityDialogLibrary::ShowEditorUtilityDialogInstance(
	UEditorUtilityDialogWidget* WidgetInstance,
	const FOnEditorUtilityDialogClosed& OnClosed,
	FText Title,
	EAppMsgType::Type ButtonLayout,
	FVector2D Size)
{
	if (!WidgetInstance)
	{
		UE_LOGF(LogEditorUtilityBlueprint, Error, "ShowEditorUtilityDialogInstance: WidgetInstance is null.");
		return;
	}

	TSharedPtr<SEditorUtilityDialog> Dialog = WrapWidgetInDialog(WidgetInstance, Title, ButtonLayout, Size);
	if (!Dialog.IsValid())
	{
		return;
	}

	WidgetInstance->OnDialogOpened();
	BindNonModalCloseCallback(*Dialog, WidgetInstance, OnClosed);
}

TPair<UEditorUtilityDialogWidget*, TSharedPtr<SEditorUtilityDialog>> UEditorUtilityDialogLibrary::CreateDialogInternal(
	TSubclassOf<UEditorUtilityDialogWidget> WidgetClass,
	const FText& Title,
	EAppMsgType::Type ButtonLayout,
	const FVector2D& Size)
{
	UEditorUtilityDialogWidget* Widget = SEditorUtilityDialog::CreateHostedWidget(WidgetClass);
	if (!Widget)
	{
		return TPair<UEditorUtilityDialogWidget*, TSharedPtr<SEditorUtilityDialog>>(nullptr, nullptr);
	}

	TSharedPtr<SEditorUtilityDialog> Dialog = WrapWidgetInDialog(Widget, Title, ButtonLayout, Size);
	return TPair<UEditorUtilityDialogWidget*, TSharedPtr<SEditorUtilityDialog>>(Widget, Dialog);
}

TSharedPtr<SEditorUtilityDialog> UEditorUtilityDialogLibrary::WrapWidgetInDialog(
	UEditorUtilityDialogWidget* Widget,
	const FText& Title,
	EAppMsgType::Type ButtonLayout,
	const FVector2D& Size)
{
	check(Widget);

	// "Construct Object of Class" in Blueprint creates the UObject but does NOT call
	// UUserWidget::Initialize(), which builds the widget tree. Without it, TakeWidget()
	// returns an empty Slate widget. CreateWidget<> calls Initialize() automatically,
	// but the instance-based path needs this explicit call.
	// Initialize() is idempotent (checks bInitialized internally), so safe to call always.
	Widget->Initialize();

	const FText EffectiveTitle = Title.IsEmpty() ? Widget->DialogTitle : Title;
	const FVector2D EffectiveSize = Size.IsNearlyZero() ? Widget->DesiredDialogSize : Size;

	TSharedRef<SEditorUtilityDialog> Dialog = SNew(SEditorUtilityDialog)
		.Title(EffectiveTitle)
		.ButtonLayout(ButtonLayout)
		.ClientSize(EffectiveSize);

	// SetContent wires ownership (SetOwningDialog) internally.
	Dialog->SetContent(Widget);

	return Dialog;
}

void UEditorUtilityDialogLibrary::BindNonModalCloseCallback(
	SEditorUtilityDialog& Dialog,
	UEditorUtilityDialogWidget* InitialWidget,
	const FOnEditorUtilityDialogClosed& OnClosed)
{
	// The widget the caller gets on close must reflect any Blueprint recompile that happened
	// while the dialog was visible. Share a weak holder between the close-lambda and the
	// rebuild-delegate: the rebuild delegate updates the holder, the close lambda reads it.
	TSharedRef<TWeakObjectPtr<UEditorUtilityDialogWidget>> WidgetHolder =
		MakeShared<TWeakObjectPtr<UEditorUtilityDialogWidget>>(InitialWidget);

	Dialog.GetOnWidgetRebuilt().AddLambda(
		[WidgetHolder](UEditorUtilityDialogWidget* NewInstance)
		{
			*WidgetHolder = NewInstance;
		});

	Dialog.Show(FOnEditorUtilityDialogSlateClosedNative::CreateLambda(
		[OnClosed, WidgetHolder](EAppReturnType::Type Result)
		{
			OnClosed.ExecuteIfBound(Result, WidgetHolder->Get());
		}));
}
