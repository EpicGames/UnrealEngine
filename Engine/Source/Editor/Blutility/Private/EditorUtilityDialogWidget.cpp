// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorUtilityDialogWidget.h"
#include "SEditorUtilityDialog.h"

void UEditorUtilityDialogWidget::CloseDialog(EAppReturnType::Type Result)
{
	if (TSharedPtr<SEditorUtilityDialog> Dialog = OwningDialog.Pin())
	{
		Dialog->CloseWithResult(Result);
	}
}

void UEditorUtilityDialogWidget::SetOwningDialog(TSharedPtr<SEditorUtilityDialog> InDialog)
{
	OwningDialog = InDialog;
}
