// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorUtilityLibrary.h"
#include "Blueprint/WidgetTree.h"
#include "EditorUtilityWidgetBlueprint.h"
#include "Engine/Selection.h"
#include "Editor.h"
#include "GameFramework/Actor.h"
#include "ContentBrowserModule.h"
#include "Modules/ModuleManager.h"
#include "IContentBrowserSingleton.h"
#include "AssetToolsModule.h"
#include "IAssetTools.h"
#include "EditorUtilitySubsystem.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Serialization/ArchiveReplaceObjectRef.h"
#include "Serialization/FindReferencersArchive.h"
#include "Templates/SubclassOf.h"
#include "WidgetBlueprint.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorUtilityLibrary)

#define LOCTEXT_NAMESPACE "BlutilityLevelEditorExtensions"

UEditorUtilityBlueprintAsyncActionBase::UEditorUtilityBlueprintAsyncActionBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UEditorUtilityBlueprintAsyncActionBase::RegisterWithGameInstance(const UObject* WorldContextObject)
{
	UEditorUtilitySubsystem* EditorUtilitySubsystem = GEditor->GetEditorSubsystem<UEditorUtilitySubsystem>();
	EditorUtilitySubsystem->RegisterReferencedObject(this);
}

void UEditorUtilityBlueprintAsyncActionBase::SetReadyToDestroy()
{
	if (UEditorUtilitySubsystem* EditorUtilitySubsystem = GEditor->GetEditorSubsystem<UEditorUtilitySubsystem>())
	{
		EditorUtilitySubsystem->UnregisterReferencedObject(this);
	}
}

UAsyncEditorDelay::UAsyncEditorDelay(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

#if WITH_EDITOR

UAsyncEditorDelay* UAsyncEditorDelay::AsyncEditorDelay(float Seconds, int32 MinimumFrames)
{
	UAsyncEditorDelay* NewTask = NewObject<UAsyncEditorDelay>();
	NewTask->Start(Seconds, MinimumFrames);

	return NewTask;
}

#endif

void UAsyncEditorDelay::Start(float InMinimumSeconds, int32 InMinimumFrames)
{
	EndFrame = GFrameCounter + InMinimumFrames;
	EndTime = FApp::GetCurrentTime() + InMinimumSeconds;
	FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateUObject(this, &UAsyncEditorDelay::HandleComplete), 0);
}

bool UAsyncEditorDelay::HandleComplete(float DeltaTime)
{
	if (FApp::GetCurrentTime() < EndTime)
	{
		return true;
	}

	if (GFrameCounter < EndFrame)
	{
		return true;
	}

	Complete.Broadcast();
	SetReadyToDestroy();
	return false;
}


UAsyncEditorWaitForGameWorld::UAsyncEditorWaitForGameWorld(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

#if WITH_EDITOR

UAsyncEditorWaitForGameWorld* UAsyncEditorWaitForGameWorld::AsyncWaitForGameWorld(int32 Index, bool Server)
{
	UAsyncEditorWaitForGameWorld* NewTask = NewObject<UAsyncEditorWaitForGameWorld>();
	NewTask->Start(Index, Server);

	return NewTask;
}

#endif

void UAsyncEditorWaitForGameWorld::Start(int32 InIndex, bool InServer)
{
	Index = InIndex;
	Server = InServer;
	FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateUObject(this, &UAsyncEditorWaitForGameWorld::OnTick), 0);
}

bool UAsyncEditorWaitForGameWorld::OnTick(float DeltaTime)
{
	if (GEditor)
	{
		int32 PIECount = 0;
		for (const FWorldContext& Context : GEngine->GetWorldContexts())
		{
			if (Context.WorldType == EWorldType::PIE)
			{
				if (UWorld* World = Context.World())
				{
					if (World->GetAuthGameMode())
					{
						// If they want the server we found it, but even if they didn't if the net mode
						// is standalone, server and client are the same, so we've found our mark
						if (Server || World->GetNetMode() == NM_Standalone)
						{
							Complete.Broadcast(World);
							SetReadyToDestroy();

							return false;
						}

						continue;
					}

					if (PIECount == Index)
					{
						Complete.Broadcast(World);
						SetReadyToDestroy();

						return false;
					}

					PIECount++;
				}
			}
		}

		return true;
	}

	Complete.Broadcast(nullptr);
	SetReadyToDestroy();

	return false;
}


UAsyncEditorOpenMapAndFocusActor::UAsyncEditorOpenMapAndFocusActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

#if WITH_EDITOR

UAsyncEditorOpenMapAndFocusActor* UAsyncEditorOpenMapAndFocusActor::AsyncEditorOpenMapAndFocusActor(FSoftObjectPath Map, FString FocusActorName)
{
	UAsyncEditorOpenMapAndFocusActor* NewTask = NewObject<UAsyncEditorOpenMapAndFocusActor>();
	NewTask->Start(Map, FocusActorName);

	return NewTask;
}

#endif

void UAsyncEditorOpenMapAndFocusActor::Start(FSoftObjectPath InMap, FString InFocusActorName)
{
	Map = InMap;
	FocusActorName = InFocusActorName;

	AddToRoot();

	UWorld* World = GEditor ? GEditor->GetEditorWorldContext(false).World() : nullptr;
	UKismetSystemLibrary::ExecuteConsoleCommand(World, FString::Printf(TEXT("Automate.OpenMapAndFocusActor %s %s"), *InMap.ToString(), *InFocusActorName));

	FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateUObject(this, &UAsyncEditorOpenMapAndFocusActor::OnTick), 0);
}

bool UAsyncEditorOpenMapAndFocusActor::OnTick(float DeltaTime)
{
	RemoveFromRoot();

	Complete.Broadcast();
	SetReadyToDestroy();

	return false;
}

UEditorUtilityLibrary::UEditorUtilityLibrary(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

#if WITH_EDITOR

TArray<AActor*> UEditorUtilityLibrary::GetSelectionSet()
{
	TArray<AActor*> Result;
	for (FSelectionIterator It(GEditor->GetSelectedActorIterator()); It; ++It)
	{
		if (AActor* Actor = Cast<AActor>(*It))
		{
			Result.Add(Actor);
		}
	}

	return Result;
}

void UEditorUtilityLibrary::GetSelectionBounds(FVector& Origin, FVector& BoxExtent, float& SphereRadius)
{
	FBoxSphereBounds::Builder BoundsBuilder;
	for (FSelectionIterator It(GEditor->GetSelectedActorIterator()); It; ++It)
	{
		if (AActor* Actor = Cast<AActor>(*It))
		{
			BoundsBuilder += Actor->GetRootComponent()->Bounds;
		}
	}

	FBoxSphereBounds Extents(BoundsBuilder);
	Origin = Extents.Origin;
	BoxExtent = Extents.BoxExtent;
	SphereRadius = (float)Extents.SphereRadius; // TODO: LWC: should be double, but need to deprecate function and replace for old C++ references to continue working.
}

TArray<UObject*> UEditorUtilityLibrary::GetSelectedAssets()
{
	return GetSelectedAssetsOfClass(UObject::StaticClass());
}

TArray<UObject*> UEditorUtilityLibrary::GetSelectedAssetsOfClass(UClass* AssetClass)
{
	//@TODO: Blocking load, no slow dialog
	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	TArray<FAssetData> SelectedAssets;
	ContentBrowserModule.Get().GetSelectedAssets(SelectedAssets);

	TArray<UObject*> Result;
	for (FAssetData& AssetData : SelectedAssets)
	{
		if (AssetData.IsInstanceOf(AssetClass))
		{
			Result.Add(AssetData.GetAsset());
		}
	}

	return Result;
}

TArray<UClass*> UEditorUtilityLibrary::GetSelectedBlueprintClasses()
{
	//@TODO: Blocking load, no slow dialog
	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	TArray<FAssetData> SelectedAssets;
	ContentBrowserModule.Get().GetSelectedAssets(SelectedAssets);

	TArray<UClass*> Result;
	for (FAssetData& AssetData : SelectedAssets)
	{
		if (TSubclassOf<UBlueprint> AssetClass = AssetData.GetClass())
		{
			if (UBlueprint* Blueprint = Cast<UBlueprint>(AssetData.GetAsset()))
			{
				Result.Add(Blueprint->GeneratedClass);
			}
		}
	}

	return Result;
}

TArray<FAssetData> UEditorUtilityLibrary::GetSelectedAssetData()
{
	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	TArray<FAssetData> SelectedAssets;
	ContentBrowserModule.Get().GetSelectedAssets(SelectedAssets);

	return SelectedAssets;
}

void UEditorUtilityLibrary::RenameAsset(UObject* Asset, const FString& NewName)
{
	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");

	TArray<FAssetRenameData> AssetsAndNames;
	const FString PackagePath = FPackageName::GetLongPackagePath(Asset->GetOutermost()->GetName());
	new (AssetsAndNames) FAssetRenameData(Asset, PackagePath, NewName);

	AssetToolsModule.Get().RenameAssetsWithDialog(AssetsAndNames);
}

AActor* UEditorUtilityLibrary::GetActorReference(FString PathToActor)
{
#if WITH_EDITOR
	return Cast<AActor>(StaticFindObject(AActor::StaticClass(), GEditor->GetEditorWorldContext().World(), *PathToActor, EFindObjectFlags::None));
#else
	return nullptr;
#endif //WITH_EDITOR
}

bool UEditorUtilityLibrary::GetCurrentContentBrowserPath(FString& OutPath)
{
	IContentBrowserSingleton& ContentBrowser = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();
	const FContentBrowserItemPath CurrentPath = ContentBrowser.GetCurrentPath();
	if (CurrentPath.HasInternalPath())
	{
		OutPath = CurrentPath.GetInternalPathString();
		return !OutPath.IsEmpty();
	}
	else
	{
		return false;
	}
}

FContentBrowserItemPath UEditorUtilityLibrary::GetCurrentContentBrowserItemPath()
{
	IContentBrowserSingleton& ContentBrowser = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();
	return ContentBrowser.GetCurrentPath();
}

TArray<FString> UEditorUtilityLibrary::GetSelectedFolderPaths()
{
	IContentBrowserSingleton& ContentBrowser = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();
	TArray<FString> Paths;
	ContentBrowser.GetSelectedFolders(Paths);
	return Paths;
}

TArray<FString> UEditorUtilityLibrary::GetSelectedPathViewFolderPaths()
{
	IContentBrowserSingleton& ContentBrowser = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();
	TArray<FString> Paths;
    ContentBrowser.GetSelectedPathViewFolders(Paths);
	return Paths;
}

void UEditorUtilityLibrary::SyncBrowserToFolders(const TArray<FString>& FolderList)
{
	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	ContentBrowserModule.Get().SyncBrowserToFolders( FolderList, false, true );
}

void UEditorUtilityLibrary::ConvertToEditorUtilityWidget(UWidgetBlueprint* WidgetBP)
{
	if (!WidgetBP)
	{
		return;
	}

	if (WidgetBP->IsA<UEditorUtilityWidgetBlueprint>())
	{
		return;
	}

	FName BPName = WidgetBP->GetFName();
	UObject* Outer = WidgetBP->GetOuter();
	EObjectFlags Flags = WidgetBP->GetFlags();

	// Rename the blueprint out of the way, create a new EUWBP, and then
	// put all of the blueprints child objects back 'under it'. The Blueprint
	// generated calss does not require any updating.

	TArray<struct FEditedDocumentInfo> OriginalEditedDocuments = WidgetBP->LastEditedDocuments;
	WidgetBP->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors | REN_SkipGeneratedClasses);
	TArray<UObject*> Children;
	GetObjectsWithOuter(WidgetBP, Children, EGetObjectsFlags::None);

	UEditorUtilityWidgetBlueprint* EWBP = NewObject<UEditorUtilityWidgetBlueprint>(Outer, BPName, Flags);
	if (EWBP->WidgetTree)
	{
		// WidgetTree is a DSO created as a side effect of construction,
		// we just want to use the existing one:
		EWBP->WidgetTree->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors);
		EWBP->WidgetTree = WidgetBP->WidgetTree;
	}
	for (UObject* Child : Children)
	{
		Child->Rename(nullptr, EWBP, REN_DontCreateRedirectors | REN_SkipGeneratedClasses | REN_AllowPackageLinkerMismatch);
	}

	UEngine::FCopyPropertiesForUnrelatedObjectsParams Params;
	Params.bPerformDuplication = true;
	Params.bNotifyObjectReplacement = false;
	Params.bPreserveRootComponent = false;
	UEngine::CopyPropertiesForUnrelatedObjects(WidgetBP, EWBP);
	EWBP->LastEditedDocuments = OriginalEditedDocuments; // mangled by UBlueprint::Rename.. unmangle

	EWBP->GeneratedClass->ClassGeneratedBy = EWBP; // update ClassGeneratedBy on the UClass
	check(EWBP->GeneratedClass == nullptr || EWBP->GeneratedClass->GetOuter() == EWBP->GetOuter());

	TMap<UObject*, UObject*> OldToNew;
	OldToNew.Add(WidgetBP, EWBP);

	// Update any references to the UBlueprint itself:

	TArray<UObject*> Targets = { WidgetBP, GetTransientPackage() };
	FArchiveReplaceObjectRef<UObject> ReplaceReferencesInRoot(EWBP, OldToNew);
	FFindReferencersArchive Archive(EWBP, Targets);
	check(Archive.GetReferenceCount(WidgetBP) == 0);

	Children.Reset();
	GetObjectsWithOuter(EWBP->GetOuter(), Children);

	for (UObject* Child : Children)
	{
		FArchiveReplaceObjectRef<UObject> ReplaceReferences(Child, OldToNew);
		FFindReferencersArchive Archive2(Child, Targets);
		check(Archive2.GetReferenceCount(WidgetBP) == 0);
	}
}

void UEditorUtilityLibrary::CastToWidgetBlueprint(UObject* Object, ECastToWidgetBlueprintCases& Branches, UWidgetBlueprint*& AsWidgetBlueprint)
{
	AsWidgetBlueprint = Cast<UWidgetBlueprint>(Object);
	Branches = (AsWidgetBlueprint == nullptr) ? ECastToWidgetBlueprintCases::CastFailed : ECastToWidgetBlueprintCases::CastSucceeded;
}

UWidget* UEditorUtilityLibrary::FindSourceWidgetByName(UWidgetBlueprint* WidgetBlueprint, FName WidgetName)
{
	UWidget* FoundWidget = nullptr;

	if (WidgetBlueprint)
	{
		WidgetBlueprint->ForEachSourceWidget([&WidgetName, &FoundWidget](UWidget* Widget)
		{
			if (Widget->GetName() == WidgetName)
			{
				FoundWidget = Widget;
			}
		});
	}

	return FoundWidget;
}

UWidget* UEditorUtilityLibrary::AddSourceWidget(UWidgetBlueprint* WidgetBlueprint, TSubclassOf<UWidget> WidgetClass, FName WidgetName, FName WidgetParentName)
{
	if (!WidgetBlueprint || !WidgetBlueprint->WidgetTree)
	{
		FFrame::KismetExecutionMessage(TEXT("Attempting to add a widget to an invalid Widget Blueprint"), ELogVerbosity::Error);
		return nullptr;
	}

	if (!WidgetClass)
	{
		FFrame::KismetExecutionMessage(TEXT("Attempting to add an invalid widget class to a Widget Blueprint"), ELogVerbosity::Error);
		return nullptr;
	}

	UPanelWidget* ParentWidget = WidgetParentName.IsNone() ? nullptr : Cast<UPanelWidget>(FindSourceWidgetByName(WidgetBlueprint, WidgetParentName));
	if (!ParentWidget && !WidgetParentName.IsNone())
	{
		FFrame::KismetExecutionMessage(TEXT("Attempting to add a widget to an invalid parent"), ELogVerbosity::Error);
		return nullptr;
	}

	if (WidgetParentName.IsNone() && WidgetBlueprint->WidgetTree->RootWidget != nullptr)
	{
		FFrame::KismetExecutionMessage(TEXT("Attempting to set the root widget when one already exists"), ELogVerbosity::Error);
		return nullptr;
	}

	const bool bDoesWidgetNameExist = FindSourceWidgetByName(WidgetBlueprint, WidgetName) != nullptr;
	const FName UniqueWidgetName = bDoesWidgetNameExist ? MakeUniqueObjectName(WidgetBlueprint->WidgetTree, WidgetClass, WidgetName) : WidgetName;
	UWidget* NewWidget = WidgetBlueprint->WidgetTree->ConstructWidget(WidgetClass, UniqueWidgetName);
	if (!NewWidget)
	{
		FFrame::KismetExecutionMessage(TEXT("Failed to create a widget to add to a Widget Blueprint"), ELogVerbosity::Error);
		return nullptr;
	}

	WidgetBlueprint->Modify();

	if (ParentWidget)
	{
		ParentWidget->AddChild(NewWidget);
	}
	else
	{
		WidgetBlueprint->WidgetTree->RootWidget = NewWidget;
	}

	WidgetBlueprint->OnVariableAdded(NewWidget->GetFName());
	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(WidgetBlueprint);
	return NewWidget;
}
#endif

#undef LOCTEXT_NAMESPACE
