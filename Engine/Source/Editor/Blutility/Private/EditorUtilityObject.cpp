// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorUtilityObject.h"

#include "EditorToolDelegates.h"
/////////////////////////////////////////////////////

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorUtilityObject)

UEditorUtilityObject::UEditorUtilityObject(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UEditorUtilityObject::PostLoad()
{
	Super::PostLoad();

	FEditorToolDelegates::OnEditorToolStarted.Broadcast(GetClass()->GetName());
}
