// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorUtilitySubsystem.h"

#include "ContentBrowserModule.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "CoreGlobals.h"
#include "Editor.h"
#include "EditorUtilityCommon.h"
#include "EditorUtilityTask.h"
#include "EditorUtilityToolMenu.h"
#include "EditorUtilityWidgetBlueprint.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintCore.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GameFramework/Actor.h"
#include "HAL/IConsoleManager.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "IBlutilityModule.h"
#include "IContentBrowserSingleton.h"
#include "Interfaces/IMainFrameModule.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "LevelEditor.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "Misc/PackageName.h"
#include "Modules/ModuleManager.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "Templates/Tuple.h"
#include "Trace/Detail/Channel.h"
#include "UObject/Class.h"
#include "UObject/Script.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/TopLevelAssetPath.h"
#include "UObject/UObjectBaseUtility.h"
#include "Widgets/Docking/SDockTab.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorUtilitySubsystem)

class FOutputDevice;
class FSubsystemCollectionBase;
class UWorld;

#define LOCTEXT_NAMESPACE "EditorUtilitySubsystem"

namespace UE::EditorUtility::Private
{

static const FName RunFunctionName = "Run";
static const FName RunOnStartupTagName = "bRunEditorUtilityOnStartup";

UClass* GetRunnableClassForAsset(UObject* Asset)
{
	if (UClass* Class = Cast<UClass>(Asset))
	{
		return Class;
	}

	if (UBlueprint* Blueprint = Cast<UBlueprint>(Asset))
	{
		return Blueprint->GeneratedClass;
	}

	return Asset->GetClass();
}

}

UEditorUtilitySubsystem::UEditorUtilitySubsystem()
	: UEditorSubsystem()
{

}

void UEditorUtilitySubsystem::Initialize(FSubsystemCollectionBase& Collection)
{

	if (!IConsoleManager::Get().FindConsoleObject(TEXT("RunTask")))
	{
		RunTaskCommandObject = IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("RunTask"),
			TEXT(""),
			FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateUObject(this, &UEditorUtilitySubsystem::RunTaskCommand),
			ECVF_Default
		);
	}

	if (!IConsoleManager::Get().FindConsoleObject(TEXT("CancelAllTasks")))
	{
		CancelAllTasksCommandObject = IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("CancelAllTasks"),
			TEXT(""),
			FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateUObject(this, &UEditorUtilitySubsystem::CancelAllTasksCommand),
			ECVF_Default
		);
	}

	IMainFrameModule& MainFrameModule = IMainFrameModule::Get();
	if (MainFrameModule.IsWindowInitialized())
	{
		HandleStartup();
	}
	else
	{
		MainFrameModule.OnMainFrameCreationFinished().AddUObject(this, &UEditorUtilitySubsystem::MainFrameCreationFinished);
	}

	TickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateUObject(this, &UEditorUtilitySubsystem::Tick), 0);

	FEditorDelegates::BeginPIE.AddUObject(this, &UEditorUtilitySubsystem::HandleOnBeginPIE);
	FEditorDelegates::EndPIE.AddUObject(this, &UEditorUtilitySubsystem::HandleOnEndPIE);

	FLevelEditorModule& LevelEditor = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	LevelEditor.OnMapChanged().AddUObject(this, &UEditorUtilitySubsystem::OnMapChanged);

	FKismetEditorUtilities::OnBlueprintGeneratedClassUnloaded.AddUObject(this, &UEditorUtilitySubsystem::OnBlueprintGeneratedClassUnloaded);
}

void UEditorUtilitySubsystem::Deinitialize()
{
	if (FModuleManager::Get().IsModuleLoaded("MainFrame"))
	{
		IMainFrameModule::Get().OnMainFrameCreationFinished().RemoveAll(this);
	}

	FTSTicker::RemoveTicker(TickerHandle);

	if (RunTaskCommandObject)
	{
		IConsoleManager::Get().UnregisterConsoleObject(RunTaskCommandObject); 
	}

	if (CancelAllTasksCommandObject)
	{
		IConsoleManager::Get().UnregisterConsoleObject(CancelAllTasksCommandObject);
	}

	FEditorDelegates::BeginPIE.RemoveAll(this);
	FEditorDelegates::EndPIE.RemoveAll(this);

	if (FLevelEditorModule* LevelEditor = FModuleManager::GetModulePtr<FLevelEditorModule>("LevelEditor"))
	{
		LevelEditor->OnMapChanged().RemoveAll(this);
	}

	FKismetEditorUtilities::OnBlueprintGeneratedClassUnloaded.RemoveAll(this);
}

void UEditorUtilitySubsystem::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(InThis, Collector);

	UEditorUtilitySubsystem* This = static_cast<UEditorUtilitySubsystem*>(InThis);
	for (auto& KVP : This->PendingTasks)
	{
		Collector.AddReferencedObjects(KVP.Value);
	}
}

void UEditorUtilitySubsystem::MainFrameCreationFinished(TSharedPtr<SWindow> InRootWindow, bool bIsRunningStartupDialog)
{
	HandleStartup();
}

void UEditorUtilitySubsystem::HandleStartup()
{
	// Handle Blueprints in the StartupObjects config list
	for (const FSoftObjectPath& ObjectPath : StartupObjects)
	{
		UObject* Object = GetValid(ObjectPath.TryLoad());
		if (!Object || Object->IsUnreachable())
		{
			UE_LOGF(LogEditorUtilityBlueprint, Warning, "Could not load: %ls", *ObjectPath.ToString());
			continue;
		}
		
		TryRun(Object);
	}

	// Handle Blueprints with bRunEditorUtilityOnStartup set to true
	if (IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();
		AssetRegistry.IsLoadingAssets())
	{
		AssetRegistry.OnFilesLoaded().AddUObject(this, &UEditorUtilitySubsystem::HandleStartupAssets);
	}
	else
	{
		HandleStartupAssets();
	}
}

void UEditorUtilitySubsystem::HandleStartupAssets()
{
	IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();
	check(!AssetRegistry.IsLoadingAssets());

	// Handle the current set of assets
	{
		FARFilter Filter;
		Filter.bRecursiveClasses = true;
		Filter.ClassPaths.Add(UBlueprintCore::StaticClass()->GetClassPathName());
		Filter.ClassPaths.Add(UBlueprintGeneratedClass::StaticClass()->GetClassPathName());
		Filter.TagsAndValues.Emplace(UE::EditorUtility::Private::RunOnStartupTagName, TEXT("True"));

		TArray<FAssetData> StartupAssets;
		AssetRegistry.GetAssets(Filter, StartupAssets);

		for (const FAssetData& StartupAsset : StartupAssets)
		{
			const FString& AssetPath = FPackageName::LongPackageNameToFilename(StartupAsset.PackagePath.ToString());
			if (FPaths::IsUnderDirectory(AssetPath, FPaths::GameDevelopersDir()))
			{
				if (!FPaths::IsUnderDirectory(AssetPath, FPaths::GameUserDeveloperDir()))
				{
					continue;
				}
			}

			UObject* Object = StartupAsset.GetAsset();
			if (!Object || Object->IsUnreachable())
			{
				UE_LOGF(LogEditorUtilityBlueprint, Warning, "Could not load: %ls", *StartupAsset.GetSoftObjectPath().ToString());
				continue;
			}

			TryRun(Object);
		}
	}

	// Watch for new assets being added
	AssetRegistry.OnAssetsAdded().AddUObject(this, &UEditorUtilitySubsystem::HandleDynamicStartupAssets);
}

void UEditorUtilitySubsystem::HandleDynamicStartupAssets(TConstArrayView<FAssetData> InAssets)
{
	for (const FAssetData& Asset : InAssets)
	{
		if (const UClass* AssetClass = Asset.GetClass();
			AssetClass && (AssetClass->IsChildOf<UBlueprintCore>() || AssetClass->IsChildOf<UBlueprintGeneratedClass>()))
		{
			if (bool bRunOnStartup = false;
				Asset.GetTagValue<bool>(UE::EditorUtility::Private::RunOnStartupTagName, bRunOnStartup) && bRunOnStartup)
			{
				UObject* Object = Asset.GetAsset();
				if (!Object || Object->IsUnreachable())
				{
					UE_LOGF(LogEditorUtilityBlueprint, Warning, "Could not load: %ls", *Asset.GetSoftObjectPath().ToString());
					continue;
				}

				TryRun(Object);
			}
		}
	}
}

void UEditorUtilitySubsystem::OnBlueprintGeneratedClassUnloaded(UBlueprintGeneratedClass* BPGC)
{
	if (BPGC->IsChildOf<UEditorUtilityToolMenuEntry>())
	{
		// Unregister any menus related to this BPGC
		TArray<UObject*> MenuInstanceObjects;
		GetObjectsOfClass(BPGC, MenuInstanceObjects, /*bIncludeDerivedClasses*/false, RF_NewerVersionExists, EInternalObjectFlags::Garbage);

		for (UObject* MenuInstanceObject : MenuInstanceObjects)
		{
			UEditorUtilityToolMenuEntry* MenuInstance = CastChecked<UEditorUtilityToolMenuEntry>(MenuInstanceObject);
			MenuInstance->UnregisterMenuEntry();
		}
	}

	ReleaseInstanceOfAsset(BPGC);
	if (BPGC->ClassGeneratedBy)
	{
		ReleaseInstanceOfAsset(BPGC->ClassGeneratedBy);
	}
}

bool UEditorUtilitySubsystem::TryRun(UObject* Asset)
{
	if (!Asset || !IsValidChecked(Asset) || Asset->IsUnreachable())
	{
		UE_LOGF(LogEditorUtilityBlueprint, Warning, "Could not run: %ls", Asset ? *Asset->GetPathName() : TEXT("None"));
		return false;
	}

	UClass* ObjectClass = UE::EditorUtility::Private::GetRunnableClassForAsset(Asset);
	if (!ObjectClass)
	{
		UE_LOGF(LogEditorUtilityBlueprint, Warning, "Missing class: %ls", *Asset->GetPathName());
		return false;
	}

	if (ObjectClass->IsChildOf(AActor::StaticClass()))
	{
		UE_LOGF(LogEditorUtilityBlueprint, Warning, "Could not run because functions on actors can only be called when spawned in a world: %ls", *Asset->GetPathName());
		return false;
	}

	UFunction* RunFunction = ObjectClass->FindFunctionByName(UE::EditorUtility::Private::RunFunctionName);
	if (RunFunction && RunFunction->ParmsSize == 0)
	{
		UObject* Instance = NewObject<UObject>(this, ObjectClass);
		ObjectInstances.Add(Asset, Instance);

		FEditorScriptExecutionGuard ScriptGuard;
		Instance->ProcessEvent(RunFunction, nullptr);
		return true;
	}
	else
	{
		UE_LOGF(LogEditorUtilityBlueprint, Warning, "Missing 0 param function named 'Run': %ls", *Asset->GetPathName());
	}

	return false;
}

bool UEditorUtilitySubsystem::TryRunClass(UClass* ObjectClass)
{
	UFunction* RunFunction = ObjectClass->FindFunctionByName(UE::EditorUtility::Private::RunFunctionName);
	if (RunFunction && RunFunction->ParmsSize == 0)
	{
		UObject* Instance = NewObject<UObject>(this, ObjectClass);
		ObjectInstances.Add(ObjectClass, Instance);

		FEditorScriptExecutionGuard ScriptGuard;
		Instance->ProcessEvent(RunFunction, nullptr);
		return true;
	}
	else
	{
		UE_LOGF(LogEditorUtilityBlueprint, Warning, "Missing 0 param function named 'Run': %ls", *ObjectClass->GetPathName());
	}

	return false;
}

bool UEditorUtilitySubsystem::CanRun(UObject* Asset) const
{
	UClass* ObjectClass = UE::EditorUtility::Private::GetRunnableClassForAsset(Asset);
	if (ObjectClass)
	{
		if (ObjectClass->IsChildOf(AActor::StaticClass()))
		{
			return false;
		}

		return true;
	}

	return false;
}

void UEditorUtilitySubsystem::ReleaseInstanceOfAsset(UObject* Asset)
{
	ObjectInstances.Remove(Asset);
}

UEditorUtilityWidget* UEditorUtilitySubsystem::SpawnAndRegisterTabAndGetID(UEditorUtilityWidgetBlueprint* InBlueprint, FName& NewTabID)
{
	FName TabID;
	RegisterTabAndGetID(InBlueprint, TabID);
	SpawnRegisteredTabByID(TabID);
	NewTabID = TabID;
	return FindUtilityWidgetFromBlueprint(InBlueprint);
}

UEditorUtilityWidget* UEditorUtilitySubsystem::SpawnAndRegisterTab(class UEditorUtilityWidgetBlueprint* InBlueprint)
{
	FName InTabID;
	return SpawnAndRegisterTabAndGetID(InBlueprint, InTabID);
}

UEditorUtilityWidget* UEditorUtilitySubsystem::SpawnAndRegisterTabWithId(class UEditorUtilityWidgetBlueprint* InBlueprint, FName InTabID)
{
	RegisterTabAndGetID(InBlueprint, InTabID);
	SpawnRegisteredTabByID(InTabID);
	return FindUtilityWidgetFromBlueprint(InBlueprint);
}

void UEditorUtilitySubsystem::RegisterTabAndGetID(class UEditorUtilityWidgetBlueprint* InBlueprint, FName& NewTabID)
{
	if (InBlueprint && !IsRunningCommandlet())
	{
		FName RegistrationName = GenerateNewTabID(InBlueprint, NewTabID);
		FText DisplayName = InBlueprint->GetTabDisplayName();
		IBlutilityModule& BlutilityModule = FModuleManager::GetModuleChecked<IBlutilityModule>("Blutility");
		FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));

		if (TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager())
		{
			// Nomad tabs need to have their tab spawner registered with the global tab manager via RegisterNomadTabSpawner
			if (InBlueprint->ShouldSpawnAsNomadTab())
			{
				if (LevelEditorTabManager->HasTabSpawner(RegistrationName))
				{
					LevelEditorTabManager->UnregisterTabSpawner(RegistrationName);
				}

				if (!FGlobalTabmanager::Get()->HasTabSpawner(RegistrationName))
				{
					FGlobalTabmanager::Get()->RegisterNomadTabSpawner(RegistrationName, FOnSpawnTab::CreateUObject(InBlueprint, &UEditorUtilityWidgetBlueprint::SpawnEditorUITab))
						.SetDisplayName(DisplayName)
						.SetGroup(BlutilityModule.GetMenuGroup().ToSharedRef());
					InBlueprint->SetRegistrationName(RegistrationName);
				}
			}
			else
			{
				if (FGlobalTabmanager::Get()->HasTabSpawner(RegistrationName))
				{
					FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(RegistrationName);
				}

				if (!LevelEditorTabManager->HasTabSpawner(RegistrationName))
				{
					LevelEditorTabManager->RegisterTabSpawner(RegistrationName, FOnSpawnTab::CreateUObject(InBlueprint, &UEditorUtilityWidgetBlueprint::SpawnEditorUITab))
						.SetDisplayName(DisplayName)
						.SetGroup(BlutilityModule.GetMenuGroup().ToSharedRef());
					InBlueprint->SetRegistrationName(RegistrationName);
				}
			}

			RegisteredTabs.Add(RegistrationName, InBlueprint);
			NewTabID = RegistrationName;
		}
	}
}

UEditorUtilityWidget* UEditorUtilitySubsystem::SpawnAndRegisterTabAndGetIDGeneratedClass(UWidgetBlueprintGeneratedClass* InGeneratedWidgetBlueprint, FName& NewTabID)
{
	FName TabID;
	RegisterTabAndGetIDGeneratedClass(InGeneratedWidgetBlueprint, TabID);
	SpawnRegisteredTabByID(TabID);
	NewTabID = TabID;

	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
	if (TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager())
	{
		if (TSharedPtr<SDockTab> FoundTab = LevelEditorTabManager->FindExistingLiveTab(NewTabID))
		{
			if (UEditorUtilityWidget **CreatedUMGWidget = SpawnedFromGeneratedClassTabs.Find(FoundTab.ToSharedRef()))
			{
				return *CreatedUMGWidget;
			}
		}
	}
	return nullptr;
}

UEditorUtilityWidget* UEditorUtilitySubsystem::SpawnAndRegisterTabGeneratedClass(UWidgetBlueprintGeneratedClass* InGeneratedWidgetBlueprint)
{
	FName InTabID;
	return SpawnAndRegisterTabAndGetIDGeneratedClass(InGeneratedWidgetBlueprint, InTabID);
}

UEditorUtilityWidget* UEditorUtilitySubsystem::SpawnAndRegisterTabWithIdGeneratedClass(UWidgetBlueprintGeneratedClass* InGeneratedWidgetBlueprint, FName InTabID)
{
	return SpawnAndRegisterTabAndGetIDGeneratedClass(InGeneratedWidgetBlueprint, InTabID);
}

void UEditorUtilitySubsystem::ExtendTabContextMenu(FMenuBuilder& InMenuBuilder, FName InTabId)
{
	InMenuBuilder.BeginSection("DockingTabAssetOptions", LOCTEXT("AssetTabSection", "Assets"));
	{
		InMenuBuilder.AddMenuEntry(
			LOCTEXT("BrowseAssetInContentBrowser", "Browse to Asset"),
			LOCTEXT("BrowseAssetInContentBrowserTooltip", "Browse to asset in content browser"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.BrowseContent"),
			FUIAction(
				FExecuteAction::CreateUObject(this, &UEditorUtilitySubsystem::OnBrowseAssetInContentBrowser, InTabId),
				FCanExecuteAction::CreateUObject(this, &UEditorUtilitySubsystem::OnCanBrowseAssetInContentBrowser, InTabId)
			)
		);
	}
	InMenuBuilder.EndSection();
}

void UEditorUtilitySubsystem::OnBrowseAssetInContentBrowser(FName InTabId)
{
	UEditorUtilityWidgetBlueprint** UtilityWidget = RegisteredTabs.Find(InTabId);
	if (UtilityWidget == nullptr || (*UtilityWidget) == nullptr)
	{
		return;
	}

	if (const FContentBrowserModule* ContentBrowserModule = FModuleManager::GetModulePtr<FContentBrowserModule>("ContentBrowser"))
	{
		const TArray<UObject*> Assets {*UtilityWidget};
		ContentBrowserModule->Get().SyncBrowserToAssets(Assets);
	}
}

bool UEditorUtilitySubsystem::OnCanBrowseAssetInContentBrowser(FName InTabId)
{
	UEditorUtilityWidgetBlueprint** UtilityWidget = RegisteredTabs.Find(InTabId);
	return UtilityWidget != nullptr && (*UtilityWidget) != nullptr;
}

void UEditorUtilitySubsystem::RegisterTabAndGetIDGeneratedClass(UWidgetBlueprintGeneratedClass* InGeneratedWidgetBlueprint, FName& NewTabID)
{
	if (InGeneratedWidgetBlueprint && !IsRunningCommandlet())
	{
		FName RegistrationName = GenerateNewTabID(InGeneratedWidgetBlueprint, NewTabID);
		FText DisplayName;
		const UEditorUtilityWidget* EditorUtilityWidget = InGeneratedWidgetBlueprint->GetDefaultObject<UEditorUtilityWidget>();

		if (EditorUtilityWidget && !EditorUtilityWidget->GetTabDisplayName().IsEmpty())
		{
			DisplayName = EditorUtilityWidget->GetTabDisplayName();
		}
		else
		{
			DisplayName = FText::FromString(FName::NameToDisplayString(InGeneratedWidgetBlueprint->GetName(), false));
		}

		FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
		if (TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager())
		{
			if (!LevelEditorTabManager->HasTabSpawner(RegistrationName))
			{
				IBlutilityModule* BlutilityModule = FModuleManager::GetModulePtr<IBlutilityModule>("Blutility");
				LevelEditorTabManager->RegisterTabSpawner(RegistrationName, FOnSpawnTab::CreateUObject(this, &UEditorUtilitySubsystem::SpawnEditorUITabFromGeneratedClass, InGeneratedWidgetBlueprint))
					.SetDisplayName(DisplayName)
					.SetGroup(BlutilityModule->GetMenuGroup().ToSharedRef());
			}

			RegisteredTabsByGeneratedClass.Add(RegistrationName, InGeneratedWidgetBlueprint);
			NewTabID = RegistrationName;
		}
	}
}

bool UEditorUtilitySubsystem::SpawnRegisteredTabByID(FName NewTabID)
{
	if (!IsRunningCommandlet())
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
		if (TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager())
		{
			// Invoke the tab via the LevelEditor's TabManager even if the tab spawner is registered globally as a nomad tab.
			// TryInvokeTab will invoke in the global tab manager if the tab is not found in the LevelEditor.
			if (TSharedPtr<SDockTab> NewDockTab = LevelEditorTabManager->TryInvokeTab(NewTabID))
			{
				NewDockTab->SetEnabled(FSlateApplication::Get().GetNormalExecutionAttribute());
				IBlutilityModule* BlutilityModule = FModuleManager::GetModulePtr<IBlutilityModule>("Blutility");
				UEditorUtilityWidgetBlueprint** WidgetToSpawn = RegisteredTabs.Find(NewTabID);
				if (WidgetToSpawn)
				{
					check(*WidgetToSpawn);
					NewDockTab->SetOnExtendContextMenu(SDockTab::FExtendContextMenu::CreateUObject(this, &UEditorUtilitySubsystem::ExtendTabContextMenu, NewTabID));
					BlutilityModule->AddLoadedScriptUI(*WidgetToSpawn);
					return true;
				}
			}
			else
			{
				UE_LOGF(LogEditorUtilityBlueprint, Error, "TryInvokeTab failed with TabId: %ls", *NewTabID.ToString());
			}
		}
	}
	return false;
}

bool UEditorUtilitySubsystem::DoesTabExist(FName NewTabID)
{
	if (!IsRunningCommandlet())
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
		if (TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager())
		{
			TSharedPtr<SDockTab> FoundTab = LevelEditorTabManager->FindExistingLiveTab(NewTabID);
			if (FoundTab)
			{
				return true;
			}
		}

		if (TSharedPtr<SDockTab> FoundNomadTab = FGlobalTabmanager::Get()->FindExistingLiveTab(NewTabID))
		{
			return true;
		}
	}

	return false;
}

bool UEditorUtilitySubsystem::CloseTabByID(FName NewTabID)
{
	if (!IsRunningCommandlet())
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
		if (TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager())
		{
			TSharedPtr<SDockTab> FoundTab = LevelEditorTabManager->FindExistingLiveTab(NewTabID);
			if (FoundTab)
			{
				FoundTab->RequestCloseTab();
				return true;
			}
		}

		if (TSharedPtr<SDockTab> FoundNomadTab = FGlobalTabmanager::Get()->FindExistingLiveTab(NewTabID))
		{
			FoundNomadTab->RequestCloseTab();
			return true;
		}
	}

	return false;
}

bool UEditorUtilitySubsystem::UnregisterTabByID(FName TabID)
{
	if (!IsRunningCommandlet())
	{
		const FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
		const TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager();
		TSharedPtr<SDockTab> FoundTab = LevelEditorTabManager->FindExistingLiveTab(TabID);

		if (!FoundTab)
		{
			FoundTab = FGlobalTabmanager::Get()->FindExistingLiveTab(TabID);
		}

		if (FoundTab)
		{
			FoundTab->RequestCloseTab();
		}

		if (RegisteredTabs.Contains(TabID))
		{
			RegisteredTabs.Remove(TabID);
			
			if (LevelEditorTabManager->HasTabSpawner(TabID))
			{
				LevelEditorTabManager->UnregisterTabSpawner(TabID);
			}

			if (FGlobalTabmanager::Get()->HasTabSpawner(TabID))
			{
				FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(TabID);
			}

			return true;
		}
	}

	return false;
}

UEditorUtilityWidget* UEditorUtilitySubsystem::FindUtilityWidgetFromBlueprint(class UEditorUtilityWidgetBlueprint* InBlueprint)
{
	if (!IsValid(InBlueprint))
	{ 
		UE_LOGF(LogEditorUtilityBlueprint, Error, "Found Invalid Blueprint in FindUtilityWidgetFromBlueprint");
		return nullptr; 
	}

	return InBlueprint->GetCreatedWidget();
}

FName UEditorUtilitySubsystem::GetTabIDFromBlueprint(class UEditorUtilityWidgetBlueprint* InBlueprint)
{
	if (!IsValid(InBlueprint))
	{ 
		UE_LOGF(LogEditorUtilityBlueprint, Error, "Found Invalid Blueprint in GetTabIDFromBlueprint");
		return NAME_None; 
	}

	return InBlueprint->GetRegistrationName();
}

bool UEditorUtilitySubsystem::Tick(float DeltaTime)
{
	UEditorUtilityTask* CurrentOrParentTask = GetActiveTask();
	TArray<TObjectPtr<UEditorUtilityTask>>* PendingChildTasks = PendingTasks.Find(CurrentOrParentTask);
	if (PendingChildTasks && PendingChildTasks->Num())
	{
		UEditorUtilityTask* PendingChildTask = (*PendingChildTasks)[0];
		PendingChildTasks->RemoveAt(0);

		StartTask(PendingChildTask);
	}

	// Canceling happens without an event in the notification it's based on checking it during tick,
	// so as we evaluate it, we check if cancel was requested, and if so, we manually trigger
	// RequestCancel, to ensure an event is fired letting the task know we want to stop.
	if (GetActiveTask() && GetActiveTask()->WasCancelRequested())
	{
		GetActiveTask()->RequestCancel();
	}

	return true;
}

void UEditorUtilitySubsystem::StartTask(UEditorUtilityTask* Task)
{
	if (!Task)
	{
		return;
	}

	ActiveTaskStack.Add(Task);

	UE_LOGF(LogEditorUtilityBlueprint, Log, "Running task %ls", *GetPathNameSafe(Task));

	// And start executing it
	Task->StartExecutingTask();
}

void UEditorUtilitySubsystem::RunTaskCommand(const TArray<FString>& Params, UWorld* InWorld, FOutputDevice& Ar)
{
	if (Params.Num() >= 1)
	{
		FString TaskName = Params[0];
		if (UClass* FoundClass = FindClassByName(TaskName))
		{
			if (!FoundClass->IsChildOf<UEditorUtilityTask>())
			{
				UE_LOGF(LogEditorUtilityBlueprint, Error, "Found Task: %ls, but it's not a subclass of 'EditorUtilityTask'.", *FoundClass->GetName());
				return;
			}

			UE_LOGF(LogEditorUtilityBlueprint, Log, "Running task %ls", *FoundClass->GetPathName());

			UEditorUtilityTask* NewTask = NewObject<UEditorUtilityTask>(this, FoundClass);
			
			//TODO Attempt to map XXX=YYY to properties on the task to make the tasks parameterizable

			RegisterAndExecuteTask(NewTask);
		}
		else
		{
			UE_LOGF(LogEditorUtilityBlueprint, Error, "Unable to find task named %ls.", *TaskName);
		}
	}
	else
	{
		UE_LOGF(LogEditorUtilityBlueprint, Error, "No task specified.  RunTask <Name of Task>");
	}
}

void UEditorUtilitySubsystem::CancelAllTasksCommand(const TArray<FString>& Params, UWorld* InWorld, FOutputDevice& Ar)
{
	PendingTasks.Reset();

	for (UEditorUtilityTask* ActiveTask : ActiveTaskStack)
	{
		ActiveTask->RequestCancel();
	}
}

void UEditorUtilitySubsystem::RegisterAndExecuteTask(UEditorUtilityTask* NewTask, UEditorUtilityTask* OptionalParentTask)
{
	if (NewTask != nullptr)
	{
		// Make sure this task wasn't already registered somehow
		ensureAlwaysMsgf(NewTask->MyTaskManager == nullptr, TEXT("RegisterAndExecuteTask(this=%s, task=%s) - Passed in task is already registered to %s"), *GetPathName(), *NewTask->GetPathName(), *GetPathNameSafe(NewTask->MyTaskManager));
		if (NewTask->MyTaskManager != nullptr)
		{
			NewTask->MyTaskManager->RemoveTaskFromActiveList(NewTask);
		}

		// Register it
		check(!(PendingTasks.Contains(NewTask) || ActiveTaskStack.Contains(NewTask)));
		for (auto& KVP : PendingTasks)
		{
			check(!KVP.Value.Contains(NewTask));
		}
		NewTask->MyTaskManager = this;
		NewTask->MyParentTask = OptionalParentTask;

		// Always append the task to the set array of tasks associated with the parent - which may be null.
		TArray<TObjectPtr<UEditorUtilityTask>>& PendingChildTasks = PendingTasks.FindOrAdd(OptionalParentTask);
		PendingChildTasks.Add(NewTask);
	}
}

void UEditorUtilitySubsystem::RemoveTaskFromActiveList(UEditorUtilityTask* Task)
{
	if (Task != nullptr)
	{
		if (ensure(Task->MyTaskManager == this))
		{
			PendingTasks.Remove(Task);
			ActiveTaskStack.Remove(Task);

			// Remove from any child set.
			for (auto& KVP : PendingTasks)
			{
				KVP.Value.Remove(Task);
			}

			Task->MyTaskManager = nullptr;

			UE_LOGF(LogEditorUtilityBlueprint, Log, "Task %ls removed", *GetPathNameSafe(Task));
		}
	}
}

void UEditorUtilitySubsystem::RegisterReferencedObject(UObject* ObjectToReference)
{
	ReferencedObjects.Add(ObjectToReference);
}

void UEditorUtilitySubsystem::UnregisterReferencedObject(UObject* ObjectToReference)
{
	ReferencedObjects.Remove(ObjectToReference);
}

UClass* UEditorUtilitySubsystem::FindClassByName(const FString& RawTargetName)
{
	FString TargetName = RawTargetName;

	// Check native classes and loaded assets first before resorting to the asset registry
	bool bIsValidClassName = true;
	if (TargetName.IsEmpty() || TargetName.Contains(TEXT(" ")))
	{
		bIsValidClassName = false;
	}
	else if (!FPackageName::IsShortPackageName(TargetName))
	{
		if (TargetName.Contains(TEXT(".")))
		{
			// Convert type'path' to just path (will return the full string if it doesn't have ' in it)
			TargetName = FPackageName::ExportTextPathToObjectPath(TargetName);

			FString PackageName;
			FString ObjectName;
			TargetName.Split(TEXT("."), &PackageName, &ObjectName);

			const bool bIncludeReadOnlyRoots = true;
			FText Reason;
			if (!FPackageName::IsValidLongPackageName(PackageName, bIncludeReadOnlyRoots, &Reason))
			{
				bIsValidClassName = false;
			}
		}
		else
		{
			bIsValidClassName = false;
		}
	}

	UClass* ResultClass = nullptr;
	if (bIsValidClassName)
	{
		ResultClass = UClass::TryFindTypeSlow<UClass>(TargetName);
	}

	// If we still haven't found anything yet, try the asset registry for blueprints that match the requirements
	if (ResultClass == nullptr)
	{
		ResultClass = FindBlueprintClass(TargetName);
	}

	return ResultClass;
}

UClass* UEditorUtilitySubsystem::FindBlueprintClass(const FString& TargetName)
{
	IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();
	if (AssetRegistry.IsLoadingAssets())
	{
		AssetRegistry.SearchAllAssets(true);
	}

	FARFilter Filter;
	Filter.bRecursiveClasses = true;
	Filter.ClassPaths.Add(UBlueprintCore::StaticClass()->GetClassPathName());
	Filter.ClassPaths.Add(UBlueprintGeneratedClass::StaticClass()->GetClassPathName());

	// We enumerate all assets to find any blueprints who inherit from native classes directly - or
	// from other blueprints.
	UClass* FoundClass = nullptr;
	AssetRegistry.EnumerateAssets(Filter, [&FoundClass, TargetName](const FAssetData& AssetData)
	{
		if ((AssetData.AssetName.ToString() == TargetName) || (AssetData.GetObjectPathString() == TargetName))
		{
			if (UObject* Asset = AssetData.GetAsset())
			{
				FoundClass = UE::EditorUtility::Private::GetRunnableClassForAsset(Asset);
				return false;
			}
		}

		return true;
	}, UE::AssetRegistry::EEnumerateAssetsFlags::AllowUnfilteredArAssets);

	return FoundClass;
}

void UEditorUtilitySubsystem::HandleOnBeginPIE(const bool bIsSimulating)
{
	OnBeginPIE.Broadcast(bIsSimulating);
}

void UEditorUtilitySubsystem::HandleOnEndPIE(const bool bIsSimulating)
{
	OnEndPIE.Broadcast(bIsSimulating);
}

TSharedRef<SDockTab> UEditorUtilitySubsystem::SpawnEditorUITabFromGeneratedClass(const FSpawnTabArgs& SpawnTabArgs, UWidgetBlueprintGeneratedClass* InGeneratedWidgetBlueprint)
{
	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.IsEnabled(FSlateApplication::Get().GetNormalExecutionAttribute());
	UEditorUtilityWidget* CreatedUMGWidget = nullptr;

	auto CreateUtilityWidgetFromGeneratedClass = [&CreatedUMGWidget, &SpawnTabArgs](UWidgetBlueprintGeneratedClass* InGeneratedWidgetBlueprint)
	{
		TSharedRef<SWidget> TabWidget = SNullWidget::NullWidget;

		UClass* BlueprintClass = InGeneratedWidgetBlueprint;
		TSubclassOf<UEditorUtilityWidget> WidgetClass = BlueprintClass;

		if (UWorld* World = GEditor->GetEditorWorldContext().World())
		{
			CreatedUMGWidget = CreateWidget<UEditorUtilityWidget>(World, WidgetClass);
			if (CreatedUMGWidget)
			{
				// Editor Utility is flagged as transient to prevent from dirty the World it's created in when a property added to the Utility Widget is changed
				// Also need to recursively mark nested utility widgets as transient to prevent them from dirtying the world (since they'll be created via CreateWidget and not CreateUtilityWidget)
				UEditorUtilityWidgetBlueprint::MarkTransientRecursive(CreatedUMGWidget);
			}
		}

		if (CreatedUMGWidget)
		{
			TabWidget = SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.HAlign(HAlign_Fill)
				[
					CreatedUMGWidget->TakeWidget()
				];

			if (UEditorUtilityWidgetBlueprint* WidgetBlueprint = Cast<UEditorUtilityWidgetBlueprint>(InGeneratedWidgetBlueprint->ClassGeneratedBy))
			{
				WidgetBlueprint->SetRegistrationName(SpawnTabArgs.GetTabId().TabType);
			}
		}
		return TabWidget;
	};

	TSharedRef<SWidget> TabWidget = CreateUtilityWidgetFromGeneratedClass(InGeneratedWidgetBlueprint);
	SpawnedTab->SetContent(TabWidget);

	SpawnedTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateUObject(this, &UEditorUtilitySubsystem::OnSpawnedFromGeneratedClassTabClosed));
	SpawnedFromGeneratedClassTabs.Add(SpawnedTab, CreatedUMGWidget);

	return SpawnedTab;
}

void UEditorUtilitySubsystem::OnSpawnedFromGeneratedClassTabClosed(TSharedRef<SDockTab> TabBeingClosed)
{
	if (UEditorUtilityWidget** Widget = SpawnedFromGeneratedClassTabs.Find(TabBeingClosed))
	{
		(*Widget)->Rename(nullptr, GetTransientPackage());
	}

	SpawnedFromGeneratedClassTabs.Remove(TabBeingClosed);
}

void UEditorUtilitySubsystem::OnMapChanged(UWorld* World, EMapChangeType MapChangeType)
{
	if (MapChangeType != EMapChangeType::SaveMap)
	{
		// We need to Delete the UMG widget if we are tearing down the World it was built with.
		for (TPair<TSharedRef<SDockTab>, UEditorUtilityWidget*> SpawnedFromGeneratedClassTab : SpawnedFromGeneratedClassTabs)
		{
			TSharedRef<SDockTab> CreatedTab = SpawnedFromGeneratedClassTab.Key;
			UEditorUtilityWidget* CreatedUMGWidget = SpawnedFromGeneratedClassTab.Value;
			if (CreatedUMGWidget && World == CreatedUMGWidget->GetWorld())
			{
				CreatedTab->SetContent(SNullWidget::NullWidget);

				CreatedUMGWidget->Rename(nullptr, GetTransientPackage());
			}
		}
		SpawnedFromGeneratedClassTabs.Empty();
	}
}

FName UEditorUtilitySubsystem::GenerateNewTabID(const UObject* WidgetObject, const FName& OptionalPostfix)
{
	static const FString DefaultPostfix = "_ActiveTab";
	return OptionalPostfix.IsNone() ? FName(*(WidgetObject->GetPathName() + DefaultPostfix)) : FName(*(WidgetObject->GetPathName() + OptionalPostfix.ToString()));
}

#undef LOCTEXT_NAMESPACE
