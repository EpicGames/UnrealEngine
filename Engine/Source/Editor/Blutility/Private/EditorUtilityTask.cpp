// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorUtilityTask.h"

#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "EditorUtilityCommon.h"
#include "EditorUtilitySubsystem.h"
#include "Engine/Engine.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/AssertionMacros.h"
#include "Misc/AsyncTaskNotification.h"
#include "Misc/Attribute.h"
#include "Trace/Detail/Channel.h"
#include "UObject/Class.h"
#include "UObject/UObjectBaseUtility.h"

//////////////////////////////////////////////////////////////////////

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorUtilityTask)

#define LOCTEXT_NAMESPACE "UEditorUtilityTask"

UEditorUtilityTask::UEditorUtilityTask()
{
}

void UEditorUtilityTask::Run()
{
	UEditorUtilitySubsystem* EditorUtilitySubsystem = GEditor->GetEditorSubsystem<UEditorUtilitySubsystem>();
	EditorUtilitySubsystem->RegisterAndExecuteTask(this, nullptr);
}

UWorld* UEditorUtilityTask::GetWorld() const
{
	if (HasAllFlags(RF_ClassDefaultObject))
	{
		// If we are a CDO, we must return nullptr instead of calling Outer->GetWorld() to fool UObject::ImplementsGetWorld.
		return nullptr;
	}

	return GEditor ? GEditor->GetEditorWorldContext(false).World() : nullptr;
}

void UEditorUtilityTask::StartExecutingTask()
{
	Cached_GIsRunningUnattendedScript = GIsRunningUnattendedScript;
	GIsRunningUnattendedScript = true;

	CreateNotification();

	BeginExecution();
	ReceiveBeginExecution();
}

void UEditorUtilityTask::FinishExecutingTask(const bool bSuccess)
{
	SetTaskNotificationText(LOCTEXT("TaskComplete", "Complete"));

	if (ensure(MyTaskManager))
	{
		MyTaskManager->RemoveTaskFromActiveList(this);
	}

	if (TaskNotification.IsValid())
	{
		TaskNotification->SetComplete(bSuccess);
		TaskNotification.Reset();
	}

	GIsRunningUnattendedScript = Cached_GIsRunningUnattendedScript;

	// Notify anyone who needs to know that we're done.
	OnFinished.Broadcast(this);
}

FText UEditorUtilityTask::GetTaskTitle() const
{
	return GetClass()->GetDisplayNameText();
}

void UEditorUtilityTask::CreateNotification()
{
	FText TaskTitle = GetTaskTitleOverride();
	if (TaskTitle.IsEmpty())
	{
		TaskTitle = GetTaskTitle();
	}
	if (TaskTitle.IsEmpty())
	{
		TaskTitle = GetClass()->GetDisplayNameText();
	}

	FAsyncTaskNotificationConfig NotificationConfig;
	NotificationConfig.TitleText = FText::Format(LOCTEXT("NotificationEditorUtilityTaskTitle", "Task {0}"), TaskTitle);
	NotificationConfig.ProgressText = LOCTEXT("Running", "Running");
	NotificationConfig.bCanCancel = true;
	TaskNotification = MakeUnique<FAsyncTaskNotification>(NotificationConfig);
}

void UEditorUtilityTask::RequestCancel()
{
	if (!bCancelRequested)
	{
		bCancelRequested = true;

		SetTaskNotificationText(LOCTEXT("TaskCanceling", "Canceling"));

		CancelRequested();
		ReceiveCancelRequested();

		FinishExecutingTask();
	}
}

bool UEditorUtilityTask::WasCancelRequested() const
{
	if (TaskNotification.IsValid())
	{
		if (TaskNotification->GetPromptAction() == EAsyncTaskNotificationPromptAction::Cancel)
		{
			return true;
		}
	}

	return bCancelRequested;
}

void UEditorUtilityTask::SetTaskNotificationText(const FText& Text)
{
	UE_LOGF(LogEditorUtilityBlueprint, Log, "%ls: %ls", *GetPathNameSafe(this), *Text.ToString());

	if (TaskNotification.IsValid())
	{
		TaskNotification->SetProgressText(Text);
	}
}

#undef LOCTEXT_NAMESPACE
