// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorUtilityWidget.h"

#include "Blueprint/WidgetTree.h"
#include "EditorToolDelegates.h"
#include "Internationalization/Internationalization.h"
#include "Misc/AssertionMacros.h"
#include "ScopedTransaction.h"
#include "UObject/Script.h"

/////////////////////////////////////////////////////

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorUtilityWidget)
#define LOCTEXT_NAMESPACE "EditorUtility"

UWidget* UEditorUtilityWidget::FindChildWidgetByName(FName WidgetName) const
{
	if (WidgetTree)
	{
		return WidgetTree->FindWidget(WidgetName);
	}

	return nullptr;
}

void UEditorUtilityWidget::ExecuteDefaultAction()
{
	check(bAutoRunDefaultAction);

	FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "BlutilityAction", "Blutility Action"));
	FEditorScriptExecutionGuard ScriptGuard;

	Run();
}

void UEditorUtilityWidget::NativeOnInitialized()
{
	Super::NativeOnInitialized();

	FEditorToolDelegates::OnEditorToolStarted.Broadcast(GetClass()->GetName());
}


#undef LOCTEXT_NAMESPACE
