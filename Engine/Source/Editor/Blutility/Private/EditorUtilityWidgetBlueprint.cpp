// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorUtilityWidgetBlueprint.h"

#include "Blueprint/UserWidget.h"
#include "Blueprint/WidgetTree.h"
#include "CoreGlobals.h"
#include "Delegates/Delegate.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "EditorUtilityWidget.h"
#include "Engine/Blueprint.h"
#include "Engine/Engine.h"
#include "Framework/Docking/TabManager.h"
#include "IBlutilityModule.h"
#include "LevelEditor.h"
#include "Modules/ModuleManager.h"
#include "SlotBase.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "Types/SlateEnums.h"
#include "UObject/Package.h"
#include "UObject/UnrealNames.h"
#include "UnrealEdMisc.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "EditorUtilityWidgetProjectSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorUtilityWidgetBlueprint)

class SWidget;
class UClass;
class UWorld;




/////////////////////////////////////////////////////
// UEditorUtilityWidgetBlueprint

void UEditorUtilityWidgetBlueprint::BeginDestroy()
{
	// Only cleanup script if it has been registered and we're not shutdowning editor
	if (!IsEngineExitRequested() && RegistrationName != NAME_None)
	{
		IBlutilityModule* BlutilityModule = FModuleManager::GetModulePtr<IBlutilityModule>("Blutility");
		if (BlutilityModule)
		{
			BlutilityModule->RemoveLoadedScriptUI(this);
		}

		FLevelEditorModule* LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>(TEXT("LevelEditor"));
		if (LevelEditorModule)
		{
			TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule->GetLevelEditorTabManager();
			if (LevelEditorTabManager.IsValid())
			{
				LevelEditorTabManager->UnregisterTabSpawner(RegistrationName);
			}
		}
	}

	Super::BeginDestroy();
}


TSharedRef<SDockTab> UEditorUtilityWidgetBlueprint::SpawnEditorUITab(const FSpawnTabArgs& SpawnTabArgs)
{
	if(bSpawnAsNomadTab)
	{
		TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab).TabRole(NomadTab);

		TSharedRef<SWidget> TabWidget = CreateUtilityWidget();
		SpawnedTab->SetContent(TabWidget);
	
		SpawnedTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateUObject(this, &UEditorUtilityWidgetBlueprint::UpdateRespawnListIfNeeded));
		CreatedTab = SpawnedTab;

		OnCompiled().Remove(OnBlueprintCompileHandle);
		OnBlueprintCompileHandle = OnCompiled().AddUObject(this, &UEditorUtilityWidgetBlueprint::RegenerateCreatedTab);
	
		FLevelEditorModule& LevelEditor = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
		LevelEditor.OnMapChanged().AddUObject(this, &UEditorUtilityWidgetBlueprint::ChangeTabWorld);

		return SpawnedTab;
	}
		
	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab);

	TSharedRef<SWidget> TabWidget = CreateUtilityWidget();
	SpawnedTab->SetContent(TabWidget);
	
	SpawnedTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateUObject(this, &UEditorUtilityWidgetBlueprint::UpdateRespawnListIfNeeded));
	CreatedTab = SpawnedTab;
	
	OnCompiled().Remove(OnBlueprintCompileHandle);
	OnBlueprintCompileHandle = OnCompiled().AddUObject(this, &UEditorUtilityWidgetBlueprint::RegenerateCreatedTab);
	
	FLevelEditorModule& LevelEditor = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	LevelEditor.OnMapChanged().AddUObject(this, &UEditorUtilityWidgetBlueprint::ChangeTabWorld);

	return SpawnedTab;
}

TSharedRef<SWidget> UEditorUtilityWidgetBlueprint::CreateUtilityWidget()
{
	TSharedRef<SWidget> TabWidget = SNullWidget::NullWidget;

	UClass* BlueprintClass = GeneratedClass;
	TSubclassOf<UEditorUtilityWidget> WidgetClass = BlueprintClass;

	if (CreatedUMGWidget)
	{
		CreatedUMGWidget->Rename(nullptr, GetTransientPackage(), REN_DoNotDirty);
	}

	CreatedUMGWidget = nullptr;
	if (UWorld* World = GEditor->GetEditorWorldContext().World())
	{
		CreatedUMGWidget = CreateWidget<UEditorUtilityWidget>(World, WidgetClass);
		if (CreatedUMGWidget)
		{
			// Editor Utility is flagged as transient to prevent from dirty the World it's created in when a property added to the Utility Widget is changed
			// Also need to recursively mark nested utility widgets as transient to prevent them from dirtying the world (since they'll be created via CreateWidget and not CreateUtilityWidget)
			MarkTransientRecursive(CreatedUMGWidget);
		}
	}

	if (CreatedUMGWidget)
	{
		TabWidget = SNew(SVerticalBox)
			.IsEnabled_UObject(this, &UEditorUtilityWidgetBlueprint::IsWidgetEnabled)
			+ SVerticalBox::Slot()
			.HAlign(HAlign_Fill)
			[
				CreatedUMGWidget->TakeWidget()
			];
	}
	return TabWidget;
}

bool UEditorUtilityWidgetBlueprint::IsWidgetEnabled() const
{
	bool bAllowPIE = bIsEnabledInPIE || (GEditor && GEditor->PlayWorld == nullptr);
	bool bAllowDebugging = bIsEnabledInDebugging || !GIntraFrameDebuggingGameThread;
	return bAllowPIE && bAllowDebugging;
}

void UEditorUtilityWidgetBlueprint::RegenerateCreatedTab(UBlueprint* RecompiledBlueprint)
{
	if (TSharedPtr<SDockTab> CreatedTabPinned = CreatedTab.Pin())
	{
		TSharedRef<SWidget> TabWidget = CreateUtilityWidget();
		CreatedTabPinned->SetContent(TabWidget);
	}
}


FText UEditorUtilityWidgetBlueprint::GetTabDisplayName() const
{
	const UEditorUtilityWidget* EditorUtilityWidget = GeneratedClass->GetDefaultObject<UEditorUtilityWidget>();

	if (EditorUtilityWidget && !EditorUtilityWidget->GetTabDisplayName().IsEmpty())
	{
		return EditorUtilityWidget->GetTabDisplayName();
	}
	return FText::FromString(FName::NameToDisplayString(GetName(), false));
}

UWidgetEditingProjectSettings* UEditorUtilityWidgetBlueprint::GetRelevantSettings()
{
	return GetMutableDefault<UEditorUtilityWidgetProjectSettings>();
}

const UWidgetEditingProjectSettings* UEditorUtilityWidgetBlueprint::GetRelevantSettings() const
{
	return GetDefault<UEditorUtilityWidgetProjectSettings>();
}

void UEditorUtilityWidgetBlueprint::ChangeTabWorld(UWorld* World, EMapChangeType MapChangeType)
{
	if (MapChangeType == EMapChangeType::TearDownWorld)
	{
		// We need to Delete the UMG widget if we are tearing down the World it was built with.
		if (CreatedUMGWidget && World == CreatedUMGWidget->GetWorld())
		{
			if (CreatedTab.IsValid())
			{
				CreatedTab.Pin()->SetContent(SNullWidget::NullWidget);
			}
			
			CreatedUMGWidget->Rename(nullptr, GetTransientPackage());
			CreatedUMGWidget = nullptr;			
		}
	}
	else if (MapChangeType != EMapChangeType::SaveMap)
	{
		// Recreate the widget if we are loading a map or opening a new map
		RegenerateCreatedTab(nullptr);
	}
}

void UEditorUtilityWidgetBlueprint::MarkTransientRecursive(UEditorUtilityWidget* UtilityWidget)
{
	UtilityWidget->SetFlags(RF_Transient);

	TArray<UWidget*> AllWidgets;
	UtilityWidget->WidgetTree->GetAllWidgets(AllWidgets);

	for (UWidget* Widget : AllWidgets)
	{
		if (UEditorUtilityWidget* ChildUtilityWidget = Cast<UEditorUtilityWidget>(Widget))
		{
			MarkTransientRecursive(ChildUtilityWidget);
			if (UPanelSlot* Slot = Widget->Slot)
			{
				Slot->SetFlags(RF_Transient);
			}
		}
	}
}

void UEditorUtilityWidgetBlueprint::UpdateRespawnListIfNeeded(TSharedRef<SDockTab> TabBeingClosed)
{
	if (GeneratedClass)
	{
		const UEditorUtilityWidget* EditorUtilityWidget = GeneratedClass->GetDefaultObject<UEditorUtilityWidget>();
		if (EditorUtilityWidget && EditorUtilityWidget->ShouldAlwaysReregisterWithWindowsMenu() == false)
		{
			IBlutilityModule* BlutilityModule = FModuleManager::GetModulePtr<IBlutilityModule>("Blutility");
			BlutilityModule->RemoveLoadedScriptUI(this);
		}
	}

	if (CreatedUMGWidget)
	{
		CreatedUMGWidget->Rename(nullptr, GetTransientPackage());
		CreatedUMGWidget = nullptr;
	}

	if (CreatedTab.IsValid())
	{
		FLevelEditorModule& LevelEditor = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
		LevelEditor.OnMapChanged().RemoveAll(this);
	}

}

void UEditorUtilityWidgetBlueprint::GetReparentingRules(TSet< const UClass* >& AllowedChildrenOfClasses, TSet< const UClass* >& DisallowedChildrenOfClasses) const
{
	AllowedChildrenOfClasses.Empty();
	AllowedChildrenOfClasses.Add(UEditorUtilityWidget::StaticClass());
}


