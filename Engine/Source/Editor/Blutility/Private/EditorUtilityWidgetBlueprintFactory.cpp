// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorUtilityWidgetBlueprintFactory.h"

#include "Blueprint/WidgetBlueprintGeneratedClass.h"
#include "Blueprint/WidgetTree.h"
#include "ClassViewerFilter.h"
#include "ClassViewerModule.h"
#include "Components/CanvasPanel.h"
#include "Components/GridPanel.h"
#include "Components/HorizontalBox.h"
#include "Components/PanelWidget.h"
#include "Components/VerticalBox.h"
#include "Components/Widget.h"
#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "EditorUtilityWidget.h"
#include "EditorUtilityWidgetBlueprint.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMisc.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Kismet2/SClassPickerDialog.h"
#include "Misc/AssertionMacros.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "Templates/Casts.h"
#include "Templates/SharedPointer.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UnrealNames.h"
#include "EditorUtilityWidgetProjectSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorUtilityWidgetBlueprintFactory)

class FFeedbackContext;

#define LOCTEXT_NAMESPACE "UEditorUtilityWidgetBlueprintFactory"

class FEditorUtilityWidgetBlueprintFactoryFilter : public IClassViewerFilter
{
public:
	/** All children of these classes will be included unless filtered out by another setting. */
	TSet< const UClass* > AllowedChildrenOfClasses;

	/** Disallowed class flags. */
	EClassFlags DisallowedClassFlags;

	bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs ) override
	{
		return !InClass->HasAnyClassFlags(DisallowedClassFlags)
			&& InFilterFuncs->IfInChildOfClassesSet(AllowedChildrenOfClasses, InClass) != EFilterReturn::Failed;
	}

	virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const IUnloadedBlueprintData > InUnloadedClassData, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
	{
		return !InUnloadedClassData->HasAnyClassFlags(DisallowedClassFlags)
			&& InFilterFuncs->IfInChildOfClassesSet(AllowedChildrenOfClasses, InUnloadedClassData) != EFilterReturn::Failed;
	}
};

/////////////////////////////////////////////////////
// UEditorUtilityWidgetBlueprintFactory

UEditorUtilityWidgetBlueprintFactory::UEditorUtilityWidgetBlueprintFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UEditorUtilityWidgetBlueprint::StaticClass();
	ParentClass = UEditorUtilityWidget::StaticClass();
}

bool UEditorUtilityWidgetBlueprintFactory::ConfigureProperties()
{
	if (GetDefault<UEditorUtilityWidgetProjectSettings>()->bUseWidgetTemplateSelector)
	{
		// Load the classviewer module to display a class picker
		FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");

		// Fill in options
		FClassViewerInitializationOptions Options;
		Options.Mode = EClassViewerMode::ClassPicker;
		Options.bShowNoneOption = true;

		TArray<TSoftClassPtr<UPanelWidget>> CommonRootWidgetClasses = GetDefault<UEditorUtilityWidgetProjectSettings>()->CommonRootWidgetClasses;
		for (int32 Index = 0; Index < CommonRootWidgetClasses.Num(); ++Index)
		{
			UClass* PanelWidgetClass = CommonRootWidgetClasses[Index].LoadSynchronous();
			if (PanelWidgetClass && PanelWidgetClass->IsChildOf(UPanelWidget::StaticClass()))
			{
				if (!Options.ExtraPickerCommonClasses.Contains(PanelWidgetClass))
				{
					Options.ExtraPickerCommonClasses.Add(PanelWidgetClass);
				}
			}
		}

		if (Options.ExtraPickerCommonClasses.Num() == 0)
		{
			Options.ExtraPickerCommonClasses.Add(UCanvasPanel::StaticClass());
		}

		TSharedPtr<FEditorUtilityWidgetBlueprintFactoryFilter> Filter = MakeShareable(new FEditorUtilityWidgetBlueprintFactoryFilter);
		Options.ClassFilters.Add(Filter.ToSharedRef());

		Filter->DisallowedClassFlags = CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists;
		Filter->AllowedChildrenOfClasses.Add(UPanelWidget::StaticClass());

		const FText TitleText = LOCTEXT("CreateRootWidgetBlueprint", "Pick Root Widget for New Editor Utility Widget");
		return SClassPickerDialog::PickClass(TitleText, Options, static_cast<UClass*&>(RootWidgetClass), UPanelWidget::StaticClass());

	}
	return true;
}

UObject* UEditorUtilityWidgetBlueprintFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	// Make sure we are trying to factory a blueprint, then create and init one
	check(Class->IsChildOf(UEditorUtilityWidgetBlueprint::StaticClass()));

	FString ParentPath = InParent->GetPathName();

	if ((ParentClass == nullptr) || !FKismetEditorUtilities::CanCreateBlueprintOfClass(ParentClass))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("ClassName"), (ParentClass != nullptr) ? FText::FromString(ParentClass->GetName()) : NSLOCTEXT("UnrealEd", "Null", "(null)"));
		FMessageDialog::Open(EAppMsgType::Ok, FText::Format(NSLOCTEXT("UnrealEd", "CannotCreateBlueprintFromClass", "Cannot create a blueprint based on the class '{ClassName}'."), Args));
		return nullptr;
	}
	else
	{
		// If the root widget selection dialog is not enabled, use a canvas panel as the root by default
		if (!GetDefault<UEditorUtilityWidgetProjectSettings>()->bUseWidgetTemplateSelector)
		{
			RootWidgetClass = UCanvasPanel::StaticClass();
		}
		UEditorUtilityWidgetBlueprint* NewBP = CastChecked<UEditorUtilityWidgetBlueprint>(FKismetEditorUtilities::CreateBlueprint(ParentClass, InParent, Name, BlueprintType, UEditorUtilityWidgetBlueprint::StaticClass(), UWidgetBlueprintGeneratedClass::StaticClass(), NAME_None));

		// Create the selected root widget
		if (NewBP->WidgetTree->RootWidget == nullptr)
		{
			if (TSubclassOf<UPanelWidget> RootWidgetPanel = RootWidgetClass)
			{
				UWidget* Root = NewBP->WidgetTree->ConstructWidget<UWidget>(RootWidgetPanel);
				NewBP->WidgetTree->RootWidget = Root;
				NewBP->OnVariableAdded(Root->GetFName());
			}
		}

		return NewBP;
	}
}

bool UEditorUtilityWidgetBlueprintFactory::CanCreateNew() const
{
	return true;
}

#undef LOCTEXT_NAMESPACE
