// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorUtilityWidgetComponents.h"

#include "Styling/DefaultStyleCache.h"
#include "Styling/UMGCoreStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorUtilityWidgetComponents)

UEditorUtilityButton::UEditorUtilityButton(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SetStyle(UE::Slate::Private::FDefaultStyleCache::GetEditor().GetButtonStyle());
}

UEditorUtilityCheckBox::UEditorUtilityCheckBox(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SetWidgetStyle(UE::Slate::Private::FDefaultStyleCache::GetEditor().GetCheckboxStyle());
}

UEditorUtilityCircularThrobber::UEditorUtilityCircularThrobber(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SetImage(UE::Slate::Private::FDefaultStyleCache::GetEditor().GetCircularThrobberBrushStyle());
}

UEditorUtilityComboBoxKey::UEditorUtilityComboBoxKey()
	: Super()
{
	SetWidgetStyle(UE::Slate::Private::FDefaultStyleCache::GetEditor().GetComboBoxStyle());
	SetItemStyle(UE::Slate::Private::FDefaultStyleCache::GetEditor().GetTableRowStyle());
	InitScrollBarStyle(UE::Slate::Private::FDefaultStyleCache::GetEditor().GetScrollBarStyle());
}

UEditorUtilityComboBoxString::UEditorUtilityComboBoxString(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SetWidgetStyle(UE::Slate::Private::FDefaultStyleCache::GetEditor().GetComboBoxStyle());
	SetItemStyle(UE::Slate::Private::FDefaultStyleCache::GetEditor().GetTableRowStyle());
	InitScrollBarStyle(UE::Slate::Private::FDefaultStyleCache::GetEditor().GetScrollBarStyle());
}

UEditorUtilityEditableText::UEditorUtilityEditableText(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SetWidgetStyle(UE::Slate::Private::FDefaultStyleCache::GetEditor().GetEditableTextStyle());
}

UEditorUtilityEditableTextBox::UEditorUtilityEditableTextBox(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	WidgetStyle = UE::Slate::Private::FDefaultStyleCache::GetEditor().GetEditableTextBoxStyle();
}

UEditorUtilityExpandableArea::UEditorUtilityExpandableArea(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SetStyle(UE::Slate::Private::FDefaultStyleCache::GetEditor().GetExpandableAreaStyle());
	SetBorderBrush(UE::Slate::Private::FDefaultStyleCache::GetEditor().GetExpandableAreaBorderBrush());
}

UEditorUtilityInputKeySelector::UEditorUtilityInputKeySelector(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SetButtonStyle(UE::Slate::Private::FDefaultStyleCache::GetEditor().GetButtonStyle());
	SetTextStyle(UE::Slate::Private::FDefaultStyleCache::GetEditor().GetTextBlockStyle());
}

UEditorUtilityListView::UEditorUtilityListView(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	WidgetStyle = UE::Slate::Private::FDefaultStyleCache::GetEditor().GetListViewStyle();
	ScrollBarStyle = UE::Slate::Private::FDefaultStyleCache::GetEditor().GetScrollBarStyle();
}

UEditorUtilityMultiLineEditableText::UEditorUtilityMultiLineEditableText(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SetWidgetStyle(UE::Slate::Private::FDefaultStyleCache::GetEditor().GetTextBlockStyle());
}

UEditorUtilityMultiLineEditableTextBox::UEditorUtilityMultiLineEditableTextBox(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	WidgetStyle = UE::Slate::Private::FDefaultStyleCache::GetEditor().GetEditableTextBoxStyle();
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TextStyle_DEPRECATED = WidgetStyle.TextStyle;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

UEditorUtilityProgressBar::UEditorUtilityProgressBar(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SetWidgetStyle(UE::Slate::Private::FDefaultStyleCache::GetEditor().GetProgressBarStyle());
}

UEditorUtilityScrollBar::UEditorUtilityScrollBar(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SetWidgetStyle(UE::Slate::Private::FDefaultStyleCache::GetEditor().GetScrollBarStyle());
}

UEditorUtilityScrollBox::UEditorUtilityScrollBox(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SetWidgetStyle(UE::Slate::Private::FDefaultStyleCache::GetEditor().GetScrollBoxStyle());
	SetWidgetBarStyle(UE::Slate::Private::FDefaultStyleCache::GetEditor().GetScrollBarStyle());
}

UEditorUtilitySlider::UEditorUtilitySlider(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SetWidgetStyle(UE::Slate::Private::FDefaultStyleCache::GetEditor().GetSliderStyle());
}

UEditorUtilitySpinBox::UEditorUtilitySpinBox(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SetWidgetStyle(UE::Slate::Private::FDefaultStyleCache::GetEditor().GetSpinBoxStyle());
}

UEditorUtilityThrobber::UEditorUtilityThrobber(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SetImage(UE::Slate::Private::FDefaultStyleCache::GetEditor().GetThrobberBrush());
}

UEditorUtilityTreeView::UEditorUtilityTreeView(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	WidgetStyle = UE::Slate::Private::FDefaultStyleCache::GetEditor().GetTreeViewStyle();
}
