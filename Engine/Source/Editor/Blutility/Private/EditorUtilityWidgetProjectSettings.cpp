// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorUtilityWidgetProjectSettings.h"
#include "WidgetBlueprint.h"
#include "WidgetCompilerRule.h"
#include "UObject/Package.h"
#include "UObject/UObjectIterator.h"

#include "Blueprint/UserWidget.h"
#include "Components/CanvasPanel.h"	
#include "Components/GridPanel.h"
#include "Components/HorizontalBox.h"
#include "Components/VerticalBox.h"
#include "../Classes/EditorUtilityWidgetBlueprint.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorUtilityWidgetProjectSettings)


FText UEditorUtilityWidgetProjectSettings::GetSectionText() const
{
	return NSLOCTEXT("EditorUtilities", "EditorUtilityWidgetsTeamSettingsName", "Editor Utility Widgets (Team)");
}

FText UEditorUtilityWidgetProjectSettings::GetSectionDescription() const
{
	return NSLOCTEXT("EditorUtilities", "EditorUtilityWidgetsTeamSettingsDescription", "Configure options for Editor Utility Widgets that affect the whole team.");
}

void UEditorUtilityWidgetProjectSettings::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	// If there's a change, we should scan for widgets currently in the error or warning state and mark them as dirty
	// so they get recompiled next time we PIE.  Don't mark all widgets dirty, or we're in for a very large recompile.
	if (PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
	{
		for (TObjectIterator<UEditorUtilityWidgetBlueprint> BlueprintIt; BlueprintIt; ++BlueprintIt)
		{
			UEditorUtilityWidgetBlueprint* Blueprint = *BlueprintIt;
			if (Blueprint->Status == BS_Error || Blueprint->Status == BS_UpToDateWithWarnings)
			{
				Blueprint->Status = BS_Dirty;
			}
		}
	}
}

FNamePermissionList& UEditorUtilityWidgetProjectSettings::GetAllowedEditorUtilityActorActions()
{
	return AllowedEditorUtilityActorActions;
}

const FNamePermissionList& UEditorUtilityWidgetProjectSettings::GetAllowedEditorUtilityActorActions() const
{
	return AllowedEditorUtilityActorActions;
}

FNamePermissionList& UEditorUtilityWidgetProjectSettings::GetAllowedEditorUtilityAssetActions()
{
	return AllowedEditorUtilityAssetActions;
}

const FNamePermissionList& UEditorUtilityWidgetProjectSettings::GetAllowedEditorUtilityAssetActions() const
{
	return AllowedEditorUtilityAssetActions;
}
