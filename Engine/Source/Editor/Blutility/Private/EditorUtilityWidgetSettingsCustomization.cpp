// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorUtilityWidgetSettingsCustomization.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "WidgetEditingProjectSettings.h"
 

TSharedRef<IDetailCustomization> FEditorUtilityWidgetSettingsCustomization::MakeInstance()
{
	return MakeShareable( new FEditorUtilityWidgetSettingsCustomization() );
}

void FEditorUtilityWidgetSettingsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	// Hide the parent class selector properties because the EditorUtilityWidgetBP factory doesn't use them yet
	TSharedPtr<IPropertyHandle> ParentSelectorProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UWidgetEditingProjectSettings, bUseUserWidgetParentClassViewerSelector), UWidgetEditingProjectSettings::StaticClass());
	ParentSelectorProperty->MarkHiddenByCustomization();
	TSharedPtr<IPropertyHandle> FavoriteParentClassesProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UWidgetEditingProjectSettings, FavoriteWidgetParentClasses), UWidgetEditingProjectSettings::StaticClass());
	FavoriteParentClassesProperty->MarkHiddenByCustomization();

	// Hide properties where we don't want the Editor Utility Widget settings to differ from the standard UMG ones
	TSharedPtr<IPropertyHandle> TemplateSelectorProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UWidgetEditingProjectSettings, bUseWidgetTemplateSelector), UWidgetEditingProjectSettings::StaticClass());
	TemplateSelectorProperty->MarkHiddenByCustomization();
	TSharedPtr<IPropertyHandle> MakeVariableProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UWidgetEditingProjectSettings, bEnableMakeVariable), UWidgetEditingProjectSettings::StaticClass());
	MakeVariableProperty->MarkHiddenByCustomization();
	TSharedPtr<IPropertyHandle> WidgetAnimationWindowProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UWidgetEditingProjectSettings, bEnableWidgetAnimationEditor), UWidgetEditingProjectSettings::StaticClass());
	WidgetAnimationWindowProperty->MarkHiddenByCustomization();
	TSharedPtr<IPropertyHandle> PaletteWindowProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UWidgetEditingProjectSettings, bEnablePaletteWindow), UWidgetEditingProjectSettings::StaticClass());
	PaletteWindowProperty->MarkHiddenByCustomization();
	TSharedPtr<IPropertyHandle> LibraryWindowProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UWidgetEditingProjectSettings, bEnableLibraryWindow), UWidgetEditingProjectSettings::StaticClass());
	LibraryWindowProperty->MarkHiddenByCustomization();
	TSharedPtr<IPropertyHandle> HierarchyWindowProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UWidgetEditingProjectSettings, bEnableHierarchyWindow), UWidgetEditingProjectSettings::StaticClass());
	HierarchyWindowProperty->MarkHiddenByCustomization();
	TSharedPtr<IPropertyHandle> BindWidgetWindowProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UWidgetEditingProjectSettings, bEnableBindWidgetWindow), UWidgetEditingProjectSettings::StaticClass());
	BindWidgetWindowProperty->MarkHiddenByCustomization();
	TSharedPtr<IPropertyHandle> NavigationSimWindowProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UWidgetEditingProjectSettings, bEnableNavigationSimulationWindow), UWidgetEditingProjectSettings::StaticClass());
	NavigationSimWindowProperty->MarkHiddenByCustomization();
	TSharedPtr<IPropertyHandle> ConfigPaletteProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UWidgetEditingProjectSettings, bUseEditorConfigPaletteFiltering), UWidgetEditingProjectSettings::StaticClass());
	ConfigPaletteProperty->MarkHiddenByCustomization();
	TSharedPtr<IPropertyHandle> ClassViewerProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UWidgetEditingProjectSettings, bUseUserWidgetParentDefaultClassViewerSelector), UWidgetEditingProjectSettings::StaticClass());
	ClassViewerProperty->MarkHiddenByCustomization();
	TSharedPtr<IPropertyHandle> UIComponentsProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UWidgetEditingProjectSettings, bEnableUIComponentsProperty), UWidgetEditingProjectSettings::StaticClass());
	UIComponentsProperty->MarkHiddenByCustomization();
}

