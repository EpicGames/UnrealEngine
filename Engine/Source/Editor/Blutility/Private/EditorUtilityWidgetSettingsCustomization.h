// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"


class IDetailLayoutBuilder;

//////////////////////////////////////////////////////////////////////////
// FEditorUtilityWidgetSettingsCustomization

class FEditorUtilityWidgetSettingsCustomization : public IDetailCustomization
{
public:
	FEditorUtilityWidgetSettingsCustomization() = default;
	virtual ~FEditorUtilityWidgetSettingsCustomization() = default;

	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	// IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
	// End of IDetailCustomization interface

private:


};
