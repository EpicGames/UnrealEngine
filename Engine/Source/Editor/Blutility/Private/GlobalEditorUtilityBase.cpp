// Copyright Epic Games, Inc. All Rights Reserved.

#include "GlobalEditorUtilityBase.h"
#include "Modules/ModuleManager.h"
#include "Misc/PackageName.h"
#include "GameFramework/Actor.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "AssetRegistry/AssetData.h"
#include "Engine/Selection.h"
#include "Editor.h"
#include "ScopedTransaction.h"
#include "IAssetTools.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "AssetToolsModule.h"

/////////////////////////////////////////////////////

#include UE_INLINE_GENERATED_CPP_BY_NAME(GlobalEditorUtilityBase)

UDEPRECATED_GlobalEditorUtilityBase::UDEPRECATED_GlobalEditorUtilityBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UWorld* UDEPRECATED_GlobalEditorUtilityBase::GetWorld() const
{
	return GEditor->GetEditorWorldContext().World();
}

TArray<AActor*> UDEPRECATED_GlobalEditorUtilityBase::GetSelectionSet()
{
	TArray<AActor*> Result;
	for (FSelectionIterator It(GEditor->GetSelectedActorIterator()); It; ++It)
	{
		if (AActor* Actor = Cast<AActor>(*It))
		{
			Result.Add(Actor);
		}
	}

	return Result;
}

void UDEPRECATED_GlobalEditorUtilityBase::GetSelectionBounds(FVector& Origin, FVector& BoxExtent, float& SphereRadius)
{
	FBoxSphereBounds::Builder BoundsBuilder;
	for (FSelectionIterator It(GEditor->GetSelectedActorIterator()); It; ++It)
	{
		if (AActor* Actor = Cast<AActor>(*It))
		{
			BoundsBuilder += Actor->GetRootComponent()->Bounds;
		}
	}

	FBoxSphereBounds Extents(BoundsBuilder);
	Origin = Extents.Origin;
	BoxExtent = Extents.BoxExtent;
	SphereRadius = (float)Extents.SphereRadius;
}

void UDEPRECATED_GlobalEditorUtilityBase::ForEachSelectedActor()
{
	TArray<AActor*> SelectionSetCache;
	for (FSelectionIterator It(GEditor->GetSelectedActorIterator()); It; ++It)
	{
		if (AActor* Actor = Cast<AActor>(*It))
		{
			SelectionSetCache.Add(Actor);
		}
	}

	int32 Index = 0;
	for (auto ActorIt = SelectionSetCache.CreateIterator(); ActorIt; ++ActorIt)
	{
		AActor* Actor = *ActorIt;
		OnEachSelectedActor.Broadcast(Actor, Index);
		++Index;
	}
}

void UDEPRECATED_GlobalEditorUtilityBase::ForEachSelectedAsset()
{
	//@TODO: Blocking load, no slow dialog
	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	TArray<FAssetData> SelectedAssets;
	ContentBrowserModule.Get().GetSelectedAssets(SelectedAssets);

	int32 Index = 0;
	for (auto AssetIt = SelectedAssets.CreateIterator(); AssetIt; ++AssetIt)
	{
		const FAssetData& AssetData = *AssetIt;
		if (UObject* Asset = AssetData.GetAsset())
		{
			OnEachSelectedAsset.Broadcast(Asset, Index);
			++Index;
		}
	}
}

TArray<UObject*> UDEPRECATED_GlobalEditorUtilityBase::GetSelectedAssets()
{
	//@TODO: Blocking load, no slow dialog
	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	TArray<FAssetData> SelectedAssets;
	ContentBrowserModule.Get().GetSelectedAssets(SelectedAssets);

	TArray<UObject*> Result;
	for (FAssetData& AssetData : SelectedAssets)
	{
		Result.Add(AssetData.GetAsset());
	}

	return Result;
}

UEditorPerProjectUserSettings* UDEPRECATED_GlobalEditorUtilityBase::GetEditorUserSettings()
{
	return GetMutableDefault<UEditorPerProjectUserSettings>();
}

void UDEPRECATED_GlobalEditorUtilityBase::ClearActorSelectionSet()
{
	GEditor->GetSelectedActors()->DeselectAll();
	bDirtiedSelectionSet = true;
}

void UDEPRECATED_GlobalEditorUtilityBase::SelectNothing()
{
	GEditor->SelectNone(true, true, false);
	bDirtiedSelectionSet = true;
}

void UDEPRECATED_GlobalEditorUtilityBase::SetActorSelectionState(AActor* Actor, bool bShouldBeSelected)
{
	GEditor->SelectActor(Actor, bShouldBeSelected, /*bNotify=*/ false);
	bDirtiedSelectionSet = true;
}

void UDEPRECATED_GlobalEditorUtilityBase::PostExecutionCleanup()
{
	if (bDirtiedSelectionSet)
	{
		GEditor->NoteSelectionChange();
		bDirtiedSelectionSet = false;
	}

	OnEachSelectedActor.Clear();
	OnEachSelectedAsset.Clear();
}

void UDEPRECATED_GlobalEditorUtilityBase::ExecuteDefaultAction()
{
	check(bAutoRunDefaultAction);

	FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "BlutilityAction", "Blutility Action") );
	FEditorScriptExecutionGuard ScriptGuard;

	OnDefaultActionClicked();
	PostExecutionCleanup();
}

void UDEPRECATED_GlobalEditorUtilityBase::RenameAsset(UObject* Asset, const FString& NewName)
{
	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");

	TArray<FAssetRenameData> AssetsAndNames;
	const FString PackagePath = FPackageName::GetLongPackagePath(Asset->GetOutermost()->GetName());
	new (AssetsAndNames) FAssetRenameData(Asset, PackagePath, NewName);

	AssetToolsModule.Get().RenameAssetsWithDialog(AssetsAndNames);
}

AActor* UDEPRECATED_GlobalEditorUtilityBase::GetActorReference(FString PathToActor)
{
#if WITH_EDITOR
	return Cast<AActor>(StaticFindObject(AActor::StaticClass(), GEditor->GetEditorWorldContext().World(), *PathToActor, EFindObjectFlags::None));
#else
	return nullptr;
#endif //WITH_EDITOR
}
