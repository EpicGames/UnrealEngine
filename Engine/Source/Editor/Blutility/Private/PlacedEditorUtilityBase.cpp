// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlacedEditorUtilityBase.h"
#include "Engine/Selection.h"
#include "Editor.h"
#include "LevelEditorViewport.h"

/////////////////////////////////////////////////////
// APlacedEditorUtilityBase

#include UE_INLINE_GENERATED_CPP_BY_NAME(PlacedEditorUtilityBase)

ADEPRECATED_PlacedEditorUtilityBase::ADEPRECATED_PlacedEditorUtilityBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PrimaryActorTick.bCanEverTick = true;
	HelpText = TEXT("Please fill out the help text");
}

void ADEPRECATED_PlacedEditorUtilityBase::TickActor(float DeltaSeconds, ELevelTick TickType, FActorTickFunction& ThisTickFunction)
{
	FEditorScriptExecutionGuard ScriptGuard;
	Super::TickActor(DeltaSeconds, TickType, ThisTickFunction);
}

TArray<AActor*> ADEPRECATED_PlacedEditorUtilityBase::GetSelectionSet()
{
	TArray<AActor*> Result;

#if WITH_EDITOR
	for (FSelectionIterator It(GEditor->GetSelectedActorIterator()); It; ++It)
	{
		if (AActor* Actor = Cast<AActor>(*It))
		{
			Result.Add(Actor);
		}
	}
#endif //WITH_EDITOR

	return Result;
}

bool ADEPRECATED_PlacedEditorUtilityBase::GetLevelViewportCameraInfo(FVector& CameraLocation, FRotator& CameraRotation)
{
	bool RetVal = false;
	CameraLocation = FVector::ZeroVector;
	CameraRotation = FRotator::ZeroRotator;

#if WITH_EDITOR
	for (FLevelEditorViewportClient* LevelVC : GEditor->GetLevelViewportClients())
	{
		if (LevelVC && LevelVC->IsPerspective())
		{
			CameraLocation = LevelVC->GetViewLocation();
			CameraRotation = LevelVC->GetViewRotation();
			RetVal = true;

			break;
		}
	}
#endif //WITH_EDITOR

	return RetVal;
}


void ADEPRECATED_PlacedEditorUtilityBase::SetLevelViewportCameraInfo(FVector CameraLocation, FRotator CameraRotation)
{

#if WITH_EDITOR
	for (FLevelEditorViewportClient* LevelVC : GEditor->GetLevelViewportClients())
	{
		if (LevelVC && LevelVC->IsPerspective())
		{
			LevelVC->SetViewLocation(CameraLocation);
			LevelVC->SetViewRotation(CameraRotation);

			break;
		}
	}
#endif //WITH_EDITOR
}

void ADEPRECATED_PlacedEditorUtilityBase::ClearActorSelectionSet()
{
	GEditor->GetSelectedActors()->DeselectAll();
	GEditor->NoteSelectionChange();
}

void ADEPRECATED_PlacedEditorUtilityBase::SelectNothing()
{
	GEditor->SelectNone(true, true, false);
}

void ADEPRECATED_PlacedEditorUtilityBase::SetActorSelectionState(AActor* Actor, bool bShouldBeSelected)
{
	GEditor->SelectActor(Actor, bShouldBeSelected, /*bNotify=*/ false);
}

AActor* ADEPRECATED_PlacedEditorUtilityBase::GetActorReference(FString PathToActor)
{
#if WITH_EDITOR
	return Cast<AActor>(StaticFindObject(AActor::StaticClass(), GEditor->GetEditorWorldContext().World(), *PathToActor, EFindObjectFlags::None));
#else
	return nullptr;
#endif //WITH_EDITOR
}
