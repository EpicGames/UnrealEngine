// Copyright Epic Games, Inc. All Rights Reserved.

#include "SEditorUtilityDialog.h"
#include "EditorUtilityDialogWidget.h"

#include "Dialog/DialogCommands.h"
#include "Editor.h"
#include "Engine/Blueprint.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/TabManager.h"
#include "Misc/App.h"
#include "Styling/AppStyle.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "EditorUtilityCommon.h"
#include "Blueprint/UserWidget.h"

#define LOCTEXT_NAMESPACE "SEditorUtilityDialog"

SEditorUtilityDialog::~SEditorUtilityDialog()
{
	UnsubscribeFromBlueprintCompile();
}

void SEditorUtilityDialog::Construct(const FArguments& InArgs)
{
	const EAppMsgType::Type MsgType = InArgs._ButtonLayout;
	EAppReturnType::Type ConfirmResult = EAppReturnType::Ok;

	switch (MsgType)
	{
	case EAppMsgType::Ok:
		ConfirmResult = EAppReturnType::Ok;
		CancelResult = EAppReturnType::Ok;
		break;
	case EAppMsgType::YesNo:
		ConfirmResult = EAppReturnType::Yes;
		CancelResult = EAppReturnType::No;
		break;
	case EAppMsgType::OkCancel:
		ConfirmResult = EAppReturnType::Ok;
		CancelResult = EAppReturnType::Cancel;
		break;
	case EAppMsgType::YesNoCancel:
		ConfirmResult = EAppReturnType::Yes;
		CancelResult = EAppReturnType::Cancel;
		break;
	case EAppMsgType::CancelRetryContinue:
		ConfirmResult = EAppReturnType::Continue;
		CancelResult = EAppReturnType::Cancel;
		break;
	case EAppMsgType::YesNoYesAllNoAll:
		ConfirmResult = EAppReturnType::Yes;
		CancelResult = EAppReturnType::No;
		break;
	case EAppMsgType::YesNoYesAllNoAllCancel:
		ConfirmResult = EAppReturnType::Yes;
		CancelResult = EAppReturnType::Cancel;
		break;
	case EAppMsgType::YesNoYesAll:
		ConfirmResult = EAppReturnType::Yes;
		CancelResult = EAppReturnType::No;
		break;
	default:
		break;
	}

	// Store the confirm result for hotkey capture. CancelResult is already a member.
	const EAppReturnType::Type LocalConfirmResult = ConfirmResult;

	// Build the command list for hotkeys (Escape = Cancel/No, Enter = Confirm/Ok/Yes).
	// CanExecuteHotkey() prevents firing after the dialog has begun closing (matching SCustomDialog pattern).
	CommandList = MakeShared<FUICommandList>();
	CommandList->MapAction(FDialogCommands::Get().Cancel,
		FUIAction(
			FSimpleDelegate::CreateSPLambda(this, [this]() { HandleButtonResult(CancelResult); }),
			FCanExecuteAction::CreateSP(this, &SEditorUtilityDialog::CanExecuteHotkey)
		)
	);
	CommandList->MapAction(FDialogCommands::Get().Confirm,
		FUIAction(
			FSimpleDelegate::CreateSPLambda(this, [this, LocalConfirmResult]() { HandleButtonResult(LocalConfirmResult); }),
			FCanExecuteAction::CreateSP(this, &SEditorUtilityDialog::CanExecuteHotkey)
		)
	);

	const FVector2D DialogSize = InArgs._ClientSize;

	SWindow::Construct(SWindow::FArguments()
		.Title(InArgs._Title)
		.ClientSize(DialogSize)
		.SizingRule(DialogSize.IsNearlyZero() ? ESizingRule::Autosized : ESizingRule::UserSized)
		.SupportsMaximize(false)
		.SupportsMinimize(false)
		[
			SNew(SBorder)
			.Padding(FMargin(4.f))
			.BorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
			[
				SAssignNew(ContentBox, SVerticalBox)

				+ SVerticalBox::Slot()
				.FillHeight(1.0f)
				[
					SNullWidget::NullWidget
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Right)
				.Padding(FMargin(0.f, 8.f, 0.f, 0.f))
				[
					CreateButtonBar(InArgs._ButtonLayout)
				]
			]
		]
	);

	GetOnWindowClosedEvent().AddSPLambda(this, [this](const TSharedRef<SWindow>& ClosedWindow)
	{
		HandleWindowClosed(ClosedWindow);
	});
}

void SEditorUtilityDialog::SetContent(UEditorUtilityDialogWidget* InWidget)
{
	check(InWidget);

	// Release the current Slate tree's reference to any prior UMG widget before we drop
	// the UObject-side pin. This is the critical ordering for the recompile-during-dialog
	// case: SListView/Details views inside the old tree hash UObject refs on paint, and
	// those refs can become stale as soon as the Blueprint reinstancer runs.
	if (ContentBox.IsValid())
	{
		ContentBox->GetSlot(0)
		[
			SNullWidget::NullWidget
		];
	}

	if (UEditorUtilityDialogWidget* OldWidget = HostedWidget.Get())
	{
		// Skip the rename when the old instance has already been replaced by the
		// reinstancer or is on the way out: Rename on a stale/torn-down UObject is the
		// hazard the rest of this function avoids. The new widget already lives in the
		// transient package via CreateWidget, so vacating the old name is unnecessary
		// in the recompile path; the guard preserves the rename for ordinary re-SetContent.
		if (!OldWidget->HasAnyFlags(RF_NewerVersionExists | RF_BeginDestroyed | RF_FinishDestroyed))
		{
			OldWidget->Rename(nullptr, GetTransientPackage(),
				REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
		}
	}

	HostedWidget.Reset(InWidget);
	InWidget->SetOwningDialog(SharedThis(this));

	if (ContentBox.IsValid())
	{
		ContentBox->GetSlot(0)
		[
			InWidget->TakeWidget()
		];
	}

	SubscribeToBlueprintCompile(InWidget);
}

UEditorUtilityDialogWidget* SEditorUtilityDialog::CreateHostedWidget(TSubclassOf<UEditorUtilityDialogWidget> WidgetClass)
{
	if (!WidgetClass)
	{
		UE_LOGF(LogEditorUtilityBlueprint, Error, "CreateHostedWidget: WidgetClass is null.");
		return nullptr;
	}

	UWorld* World = GEditor ? GEditor->GetEditorWorldContext().World() : nullptr;
	if (!World)
	{
		UE_LOGF(LogEditorUtilityBlueprint, Error, "CreateHostedWidget: No editor world available.");
		return nullptr;
	}

	UEditorUtilityDialogWidget* Widget = CreateWidget<UEditorUtilityDialogWidget>(World, WidgetClass);
	if (!Widget)
	{
		UE_LOGF(LogEditorUtilityBlueprint, Error, "CreateHostedWidget: Failed to create widget of class %ls.", *WidgetClass->GetName());
		return nullptr;
	}

	Widget->SetFlags(RF_Transient);
	return Widget;
}

void SEditorUtilityDialog::SubscribeToBlueprintCompile(UEditorUtilityDialogWidget* ForWidget)
{
	UnsubscribeFromBlueprintCompile();

	if (!ForWidget)
	{
		return;
	}

	// Classes produced by Blueprint compilation carry a back-pointer to their source UBlueprint.
	// Native-only classes (or cooked runtime classes) do not, in which case there is nothing to
	// recompile against - skip the subscription silently.
	UBlueprint* Blueprint = Cast<UBlueprint>(ForWidget->GetClass()->ClassGeneratedBy);
	if (!Blueprint)
	{
		return;
	}

	WatchedBlueprint = Blueprint;
	OnBlueprintCompiledHandle = Blueprint->OnCompiled().AddSP(this, &SEditorUtilityDialog::HandleHostedBlueprintCompiled);
}

void SEditorUtilityDialog::UnsubscribeFromBlueprintCompile()
{
	if (UBlueprint* Blueprint = WatchedBlueprint.Get())
	{
		Blueprint->OnCompiled().Remove(OnBlueprintCompiledHandle);
	}
	OnBlueprintCompiledHandle.Reset();
	WatchedBlueprint.Reset();
}

void SEditorUtilityDialog::HandleHostedBlueprintCompiled(UBlueprint* RecompiledBlueprint)
{
	// A compile event can still fire after the user has dismissed the dialog but before
	// the SWindow tears down. Ignore it - HandleWindowClosed already unsubscribed, but
	// be defensive in case the order flips.
	if (bIsClosing)
	{
		return;
	}

	// Avoid PVS V623: a `?:` with a TObjectPtr<UClass> on one side and `nullptr` on the
	// other forces construction of a TSubclassOf temporary. Use if/else explicitly.
	UClass* NewClass = nullptr;
	if (RecompiledBlueprint)
	{
		NewClass = RecompiledBlueprint->GeneratedClass;
	}

	UEditorUtilityDialogWidget* NewWidget = CreateHostedWidget(NewClass);
	if (!NewWidget)
	{
		// Compile produced no usable class, or editor world is gone. Clear the stale
		// widget tree BEFORE initiating close, so any paint pass between here and
		// window destruction cannot hash zombie TObjectPtrs from the pre-reinstance tree.
		if (ContentBox.IsValid())
		{
			ContentBox->GetSlot(0)
			[
				SNullWidget::NullWidget
			];
		}

		// Drop our reference to the (now-reinstanced) old widget BEFORE initiating close.
		// HandleButtonResult and HandleWindowClosed both invoke OnDialogButtonPressed/
		// OnDialogCancelled on HostedWidget.Get(); calling Blueprint events on a
		// reinstanced UObject is the exact hazard the rest of this change prevents.
		HostedWidget.Reset();

		// Tell external listeners (e.g. the non-modal close-callback's holder) that the
		// dialog no longer holds a current widget, so they pass nullptr to consumers
		// rather than report the just-reinstanced pre-compile pointer.
		OnWidgetRebuiltDelegate.Broadcast(nullptr);

		HandleButtonResult(CancelResult);
		return;
	}

	SetContent(NewWidget);
	NewWidget->OnDialogOpened();
	// Broadcast AFTER OnDialogOpened so external listeners observe an already-initialised
	// rebuilt widget, not one mid-construction.
	OnWidgetRebuiltDelegate.Broadcast(NewWidget);
}

EAppReturnType::Type SEditorUtilityDialog::ShowModal()
{
	if (!FApp::IsUnattended() && !GIsRunningUnattendedScript && FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().AddModalWindow(
			SharedThis(this),
			FGlobalTabmanager::Get()->GetRootWindow());
	}
	else
	{
		// In unattended mode the window was never shown, so HandleWindowClosed won't fire.
		// Fire widget close events manually to maintain symmetry with OnDialogOpened.
		if (UEditorUtilityDialogWidget* Widget = HostedWidget.Get())
		{
			Widget->OnDialogButtonPressed(CancelResult);
			Widget->OnDialogCancelled();
		}
		UE_LOGF(LogEditorUtilityBlueprint, Warning, "SEditorUtilityDialog::ShowModal called in unattended mode, returning default Cancel.");
	}

	return LastPressedButton;
}

void SEditorUtilityDialog::Show(FOnEditorUtilityDialogSlateClosedNative InOnClosed)
{
	OnClosedDelegate = InOnClosed;

	if (!FApp::IsUnattended() && !GIsRunningUnattendedScript && FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().AddWindow(
			SharedThis(this), true);
	}
	else
	{
		OnClosedDelegate.ExecuteIfBound(LastPressedButton);
		UE_LOGF(LogEditorUtilityBlueprint, Warning, "SEditorUtilityDialog::Show called in unattended mode, firing OnClosed immediately.");
	}
}

void SEditorUtilityDialog::CloseWithResult(EAppReturnType::Type Result)
{
	HandleButtonResult(Result);
}

FReply SEditorUtilityDialog::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	return SWindow::OnKeyDown(MyGeometry, InKeyEvent);
}

TSharedRef<SWidget> SEditorUtilityDialog::CreateButtonBar(EAppMsgType::Type MsgType)
{
	TSharedRef<SHorizontalBox> ButtonBox = SNew(SHorizontalBox);

	auto AddButton = [this, &ButtonBox](const FText& Label, EAppReturnType::Type Result, bool bIsPrimary = false)
	{
		const FButtonStyle* Style = bIsPrimary
			? &FAppStyle::Get().GetWidgetStyle<FButtonStyle>("PrimaryButton")
			: &FAppStyle::Get().GetWidgetStyle<FButtonStyle>("Button");

		ButtonBox->AddSlot()
			.AutoWidth()
			.Padding(FAppStyle::Get().GetMargin("StandardDialog.SlotPadding"))
			[
				SNew(SButton)
				.ButtonStyle(Style)
				.OnClicked_Lambda([this, Result]() { return OnButtonClicked(Result); })
				[
					SNew(STextBlock)
					.Text(Label)
				]
			];
	};

	switch (MsgType)
	{
	case EAppMsgType::Ok:
		AddButton(LOCTEXT("Ok", "OK"), EAppReturnType::Ok, true);
		break;
	case EAppMsgType::YesNo:
		AddButton(LOCTEXT("Yes", "Yes"), EAppReturnType::Yes, true);
		AddButton(LOCTEXT("No", "No"), EAppReturnType::No);
		break;
	case EAppMsgType::OkCancel:
		AddButton(LOCTEXT("Ok", "OK"), EAppReturnType::Ok, true);
		AddButton(LOCTEXT("Cancel", "Cancel"), EAppReturnType::Cancel);
		break;
	case EAppMsgType::YesNoCancel:
		AddButton(LOCTEXT("Yes", "Yes"), EAppReturnType::Yes, true);
		AddButton(LOCTEXT("No", "No"), EAppReturnType::No);
		AddButton(LOCTEXT("Cancel", "Cancel"), EAppReturnType::Cancel);
		break;
	case EAppMsgType::CancelRetryContinue:
		AddButton(LOCTEXT("Continue", "Continue"), EAppReturnType::Continue, true);
		AddButton(LOCTEXT("Retry", "Retry"), EAppReturnType::Retry);
		AddButton(LOCTEXT("Cancel", "Cancel"), EAppReturnType::Cancel);
		break;
	case EAppMsgType::YesNoYesAllNoAll:
		AddButton(LOCTEXT("Yes", "Yes"), EAppReturnType::Yes, true);
		AddButton(LOCTEXT("No", "No"), EAppReturnType::No);
		AddButton(LOCTEXT("YesAll", "Yes All"), EAppReturnType::YesAll);
		AddButton(LOCTEXT("NoAll", "No All"), EAppReturnType::NoAll);
		break;
	case EAppMsgType::YesNoYesAllNoAllCancel:
		AddButton(LOCTEXT("Yes", "Yes"), EAppReturnType::Yes, true);
		AddButton(LOCTEXT("No", "No"), EAppReturnType::No);
		AddButton(LOCTEXT("YesAll", "Yes All"), EAppReturnType::YesAll);
		AddButton(LOCTEXT("NoAll", "No All"), EAppReturnType::NoAll);
		AddButton(LOCTEXT("Cancel", "Cancel"), EAppReturnType::Cancel);
		break;
	case EAppMsgType::YesNoYesAll:
		AddButton(LOCTEXT("Yes", "Yes"), EAppReturnType::Yes, true);
		AddButton(LOCTEXT("No", "No"), EAppReturnType::No);
		AddButton(LOCTEXT("YesAll", "Yes All"), EAppReturnType::YesAll);
		break;
	default:
		AddButton(LOCTEXT("Ok", "OK"), EAppReturnType::Ok, true);
		break;
	}

	return ButtonBox;
}

FReply SEditorUtilityDialog::OnButtonClicked(EAppReturnType::Type Result)
{
	HandleButtonResult(Result);
	return FReply::Handled();
}

void SEditorUtilityDialog::HandleButtonResult(EAppReturnType::Type Result)
{
	if (bIsClosing)
	{
		return;
	}
	bIsClosing = true;

	LastPressedButton = Result;

	if (UEditorUtilityDialogWidget* Widget = HostedWidget.Get())
	{
		Widget->OnDialogButtonPressed(Result);

		if (IsPositiveResult(Result))
		{
			Widget->OnDialogConfirmed();
		}
		else
		{
			Widget->OnDialogCancelled();
		}
	}

	RequestDestroyWindow();
}

void SEditorUtilityDialog::HandleWindowClosed(const TSharedRef<SWindow>& /*ClosedWindow*/)
{
	// Stop listening for Blueprint compiles before firing close callbacks - otherwise a
	// compile that lands during teardown could rebuild content into a window that is
	// already unwinding.
	UnsubscribeFromBlueprintCompile();

	if (!bIsClosing)
	{
		bIsClosing = true;
		LastPressedButton = CancelResult;

		if (UEditorUtilityDialogWidget* Widget = HostedWidget.Get())
		{
			Widget->OnDialogButtonPressed(CancelResult);
			Widget->OnDialogCancelled();
		}
	}

	OnClosedDelegate.ExecuteIfBound(LastPressedButton);
}

bool SEditorUtilityDialog::CanExecuteHotkey() const
{
	return !bIsClosing;
}

bool SEditorUtilityDialog::IsPositiveResult(EAppReturnType::Type Result)
{
	switch (Result)
	{
	case EAppReturnType::Ok:
	case EAppReturnType::Yes:
	case EAppReturnType::YesAll:
	case EAppReturnType::Retry:
	case EAppReturnType::Continue:
		return true;
	default:
		return false;
	}
}

#undef LOCTEXT_NAMESPACE
