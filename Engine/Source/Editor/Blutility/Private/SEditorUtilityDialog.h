// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GenericPlatform/GenericPlatformMisc.h"
#include "Framework/Commands/UICommandList.h"
#include "Templates/SubclassOf.h"
#include "UObject/StrongObjectPtr.h"
#include "UObject/WeakObjectPtr.h"
#include "Widgets/SWindow.h"

class UBlueprint;
class UEditorUtilityDialogWidget;

/** Delegate fired when an SEditorUtilityDialog is closed (used by the non-modal Show path). */
DECLARE_DELEGATE_OneParam(FOnEditorUtilityDialogSlateClosedNative, EAppReturnType::Type /*Result*/);

/**
 * Multicast delegate fired when the hosted widget is rebuilt after a Blueprint recompile.
 * Callers that cached the widget pointer should refresh it. Also fired with nullptr when the
 * dialog cancels because the recompile produced no usable class, so listeners can drop their
 * stale reference rather than report it back to consumers.
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnEditorUtilityDialogWidgetRebuilt, UEditorUtilityDialogWidget* /*NewInstance*/);

/**
 * A modal (or non-modal) dialog window that hosts an Editor Utility Widget.
 *
 * Modeled after SCustomDialog but specialized for EUW content. Auto-generates
 * a standard button bar from EAppMsgType and returns EAppReturnType results.
 */
class SEditorUtilityDialog : public SWindow
{
public:
	SLATE_BEGIN_ARGS(SEditorUtilityDialog)
		: _ButtonLayout(EAppMsgType::OkCancel)
		, _ClientSize(FVector2D::ZeroVector)
	{}
		SLATE_ARGUMENT(FText, Title)
		SLATE_ARGUMENT(EAppMsgType::Type, ButtonLayout)
		SLATE_ARGUMENT(FVector2D, ClientSize)
	SLATE_END_ARGS()

	virtual ~SEditorUtilityDialog() override;

	void Construct(const FArguments& InArgs);

	/**
	 * Host the given widget in this dialog. Safe to call multiple times:
	 * tears down any prior hosted widget, installs the new one, wires ownership,
	 * and (re)subscribes to the widget's Blueprint compile delegate so the dialog
	 * rebuilds its content when the user recompiles the widget Blueprint.
	 */
	void SetContent(UEditorUtilityDialogWidget* InWidget);

	EAppReturnType::Type ShowModal();
	void Show(FOnEditorUtilityDialogSlateClosedNative OnClosed);
	void CloseWithResult(EAppReturnType::Type Result);
	EAppReturnType::Type GetResult() const { return LastPressedButton; }

	/** Current hosted widget, or nullptr. After a Blueprint recompile this returns the rebuilt instance. */
	UEditorUtilityDialogWidget* GetHostedWidget() const { return HostedWidget.Get(); }

	/** Fires after the hosted widget has been rebuilt due to a Blueprint recompile. */
	FOnEditorUtilityDialogWidgetRebuilt& GetOnWidgetRebuilt() { return OnWidgetRebuiltDelegate; }

	/**
	 * Create a dialog widget instance suitable for hosting in an SEditorUtilityDialog.
	 * Single source of truth for world resolution, CreateWidget, RF_Transient flagging,
	 * and error logging. Returns nullptr and logs on failure.
	 */
	static UEditorUtilityDialogWidget* CreateHostedWidget(TSubclassOf<UEditorUtilityDialogWidget> WidgetClass);

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

private:
	TSharedRef<SWidget> CreateButtonBar(EAppMsgType::Type MsgType);
	FReply OnButtonClicked(EAppReturnType::Type Result);
	void HandleButtonResult(EAppReturnType::Type Result);
	void HandleWindowClosed(const TSharedRef<SWindow>& ClosedWindow);
	bool CanExecuteHotkey() const;
	static bool IsPositiveResult(EAppReturnType::Type Result);

	/** Subscribe to the hosted widget's UBlueprint::OnCompiled so we can rebuild on recompile. Idempotent. */
	void SubscribeToBlueprintCompile(UEditorUtilityDialogWidget* ForWidget);

	/** Unsubscribe from the currently-watched UBlueprint's OnCompiled delegate, if any. */
	void UnsubscribeFromBlueprintCompile();

	/** Rebuild-on-compile handler. Replaces the hosted widget with a fresh instance of the recompiled class. */
	void HandleHostedBlueprintCompiled(UBlueprint* RecompiledBlueprint);

	EAppReturnType::Type LastPressedButton = EAppReturnType::Cancel;
	EAppReturnType::Type CancelResult = EAppReturnType::Cancel;
	bool bIsClosing = false;
	TStrongObjectPtr<UEditorUtilityDialogWidget> HostedWidget;
	TSharedPtr<SVerticalBox> ContentBox;
	TSharedPtr<FUICommandList> CommandList;
	FOnEditorUtilityDialogSlateClosedNative OnClosedDelegate;

	TWeakObjectPtr<UBlueprint> WatchedBlueprint;
	FDelegateHandle OnBlueprintCompiledHandle;
	FOnEditorUtilityDialogWidgetRebuilt OnWidgetRebuiltDelegate;
};
