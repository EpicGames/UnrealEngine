// Copyright Epic Games, Inc. All Rights Reserved.


#include "ToolMenuWidget.h"
#include "ToolMenus.h"
#include "Widgets/Layout/SSpacer.h"
#include "EditorUtilityWidgetBlueprint.h"
#include "EditorUtilityWidget.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ToolMenuWidget)



UToolMenuWidget::UToolMenuWidget(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, MenuName("MenuBar")
	, MenuType(EMultiBoxType::MenuBar)
{
	if (MyToolMenu == SNullWidget::NullWidget)
	{
		MyToolMenu = SNew(SSpacer);
	}
	UpdateFullMenuName();
}

TSharedRef<SWidget> UToolMenuWidget::RebuildWidget()
{
	if (!UToolMenus::Get()->IsMenuRegistered(FullMenuName))
	{
		UToolMenu* ToolBar = UToolMenus::Get()->RegisterMenu(FullMenuName, NAME_None, MenuType);
	}

	FToolMenuContext MenuContext;
	MyToolMenu = UToolMenus::Get()->GenerateWidget(FullMenuName, MenuContext);
	return MyToolMenu;
}

void UToolMenuWidget::UpdateFullMenuName()
{
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		return;
	}

	// PostEditChangeProperty is called on the transient and the class object.
	// To get a consistent name representing the containing editor utility, either the EditorUtilityWidgetBlueprint or the EditorUtilityWidget needs to be used.
	UEditorUtilityWidgetBlueprint* EUWBlueprintOuter = GetTypedOuter<UEditorUtilityWidgetBlueprint>();
	UEditorUtilityWidget* EUWOuter = GetTypedOuter<UEditorUtilityWidget>();

	if (EUWBlueprintOuter)
	{
		FullMenuName = FName(*(FString::Format(TEXT("{0}.{1}"), { *(EUWBlueprintOuter->GetName()), *MenuName })));
	}
	else if (EUWOuter)
	{
		FString PackageName = EUWOuter->GetArchetype()->GetPackage()->GetName();
		FullMenuName = FName(*(FString::Format(TEXT("{0}.{1}"), { *(FPaths::GetBaseFilename(PackageName)), *MenuName })));
	}
}

#if WITH_EDITOR

void UToolMenuWidget::PostInitProperties()
{
	Super::PostInitProperties();

	if (IsValidChecked(this))
	{
		UpdateFullMenuName();
	}
}

void UToolMenuWidget::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property)
	{
		static const FName MenuNameProp("MenuName");

		FName PropertyName = PropertyChangedEvent.Property->GetFName();

		if (PropertyName == MenuNameProp)
		{
			UpdateFullMenuName();
		}
	}
	
}

#endif //WITH_EDITOR
