// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "EditorUtilityTask.h"
#include "HAL/Platform.h"
#include "Kismet/BlueprintAsyncActionBase.h"
#include "Templates/SubclassOf.h"
#include "Templates/UniquePtr.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "AsyncCaptureScene.generated.h"

class ASceneCapture2D;
class UCameraComponent;
class UObject;
class UTextureRenderTarget2D;
struct FFrame;
template<typename PixelType>
struct TImagePixelData;

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnAsyncCaptureSceneComplete, UTextureRenderTarget2D*, Texture);

UCLASS(MinimalAPI)
class UAsyncCaptureScene : public UBlueprintAsyncActionBase
{
	GENERATED_BODY()

public:
	UAsyncCaptureScene();

	UFUNCTION(BlueprintCallable, meta=( BlueprintInternalUseOnly="true" ))
	static UAsyncCaptureScene* CaptureSceneAsync(UCameraComponent* ViewCamera, TSubclassOf<ASceneCapture2D> SceneCaptureClass, int ResX, int ResY);
	
	UFUNCTION(BlueprintCallable, meta = (BlueprintInternalUseOnly = "true"))
	static UAsyncCaptureScene* CaptureSceneWithWarmupAsync(UCameraComponent* ViewCamera, TSubclassOf<ASceneCapture2D> SceneCaptureClass, int ResX, int ResY, int WarmUpFrames);

	virtual void Activate() override;
public:

	UPROPERTY(BlueprintAssignable)
	FOnAsyncCaptureSceneComplete Complete;

private:

	void NotifyComplete(UTextureRenderTarget2D* InTexture);
	void Start(UCameraComponent* ViewCamera, TSubclassOf<ASceneCapture2D> SceneCaptureClass, int ResX, int ResY, int InWarmUpFrames);
	void FinishLoadingBeforeScreenshot();
	
private:
	UPROPERTY()
	TObjectPtr<ASceneCapture2D> SceneCapture;

	UPROPERTY()
	TObjectPtr<UTextureRenderTarget2D> SceneCaptureRT;

	int32 WarmUpFrames;
};
