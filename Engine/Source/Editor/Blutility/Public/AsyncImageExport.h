// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "EditorUtilityTask.h"
#include "Kismet/BlueprintAsyncActionBase.h"
#include "Math/UnrealMathSSE.h"
#include "Templates/SubclassOf.h"
#include "Templates/UniquePtr.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "AsyncImageExport.generated.h"

class UObject;
class UTexture;
class UTextureRenderTarget2D;
struct FColor;
struct FFrame;

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnExportImageAsyncComplete, bool, bSuccess);

UCLASS(MinimalAPI)
class UAsyncImageExport : public UBlueprintAsyncActionBase
{
	GENERATED_BODY()

public:
	UAsyncImageExport();

	UFUNCTION(BlueprintCallable, meta=( BlueprintInternalUseOnly="true" ))
	static UAsyncImageExport* ExportImageAsync(UTexture* Texture, const FString& OutputFile, int Quality = 100);

	virtual void Activate() override;
public:

	UPROPERTY(BlueprintAssignable)
	FOnExportImageAsyncComplete Complete;

private:

	void NotifyComplete(bool bSuccess);
	void Start(UTexture* Texture, const FString& OutputFile);
	void ReadPixelsFromRT(UTextureRenderTarget2D* InRT, TArray<FColor>* OutPixels);
	void ExportImage(TArray<FColor>&& RawPixels, FIntPoint ImageSize);

private:
	UPROPERTY()
	TObjectPtr<UTexture> TextureToExport;

	UPROPERTY()
	int32 Quality;

	UPROPERTY()
	FString TargetFile;
};
