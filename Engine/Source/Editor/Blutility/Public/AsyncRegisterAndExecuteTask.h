// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorUtilityTask.h"
#include "Kismet/BlueprintAsyncActionBase.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "AsyncRegisterAndExecuteTask.generated.h"

class UObject;
struct FFrame;

UCLASS(MinimalAPI)
class UAsyncRegisterAndExecuteTask : public UBlueprintAsyncActionBase
{
	GENERATED_BODY()

public:
	UAsyncRegisterAndExecuteTask();

	UFUNCTION(BlueprintCallable, meta=( BlueprintInternalUseOnly="true" ))
	static UAsyncRegisterAndExecuteTask* RegisterAndExecuteTask(UEditorUtilityTask* Task, UEditorUtilityTask* OptionalParentTask = nullptr);

public:

	UPROPERTY(BlueprintAssignable)
	FOnEditorUtilityTaskDynamicDelegate OnFinished;

public:

	void Start(UEditorUtilityTask* Task, UEditorUtilityTask* OptionalParentTask);

private:

	void HandleFinished(UEditorUtilityTask* Task);
};
