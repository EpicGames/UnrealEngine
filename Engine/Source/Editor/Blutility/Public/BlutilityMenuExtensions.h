// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorUtilityAssetPrototype.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "HAL/Platform.h"
#include "Templates/Function.h"
#include "UObject/Interface.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/TopLevelAssetPath.h"
#include "UObject/UObjectGlobals.h"

class AActor;
class FMenuBuilder;
class FProperty;
class FString;
class FText;
class IEditorUtilityExtension;
struct FToolMenuSection;
class UFunction;
class UObject;
class FAssetActionUtilityPrototype;

#define UE_API BLUTILITY_API

// Blutility Menu extension helpers
class FBlutilityMenuExtensions
{
public:
	/** Helper function to get all Blutility classes derived from the specified class name */
	static UE_API void GetBlutilityClasses(TArray<FAssetData>& OutAssets, FTopLevelAssetPath InClassName);

	/** Helper function that populates a menu based on the exposed functions in a set of Blutility objects */
	static void CreateActorBlutilityActionsMenu(FToolMenuSection& InSection, TMap<TSharedRef<FAssetActionUtilityPrototype>, TSet<int32>> Utils, const TArray<AActor*> SelectedSupportedActors);
	static void CreateAssetBlutilityActionsMenu(FToolMenuSection& InSection, TMap<TSharedRef<FAssetActionUtilityPrototype>, TSet<int32>> Utils, const TArray<FAssetData> SelectedSupportedAssets);

	/* Structure that defines parameters related to the menu passed into the CreateObjectBlutilityActionsMenu function. */
	struct FMenuParams
	{
		FText Label;
		FText ToolTip;
		FName IconName = "GraphEditor.Event_16x";
		bool bAsSubMenu = true;
	};
		
	/* Structure that defines the parameters related to actions and the objects to act on passed into the CreateObjectBlutilityActionsMenu function. */
	struct FObjectActionParams
	{
		TArray<UObject*> Objects;
		TMap<TSharedRef<FAssetActionUtilityPrototype>, TSet<int32>> Utils;
		TFunction<bool(const FProperty* Property)> IsValidPropertyFunc = [](const FProperty*){ return true; };
	};
	
	/** Helper function that populates a menu based on the exposed functions in a set of Blutility objects */
	static UE_API void CreateObjectBlutilityActionsMenu(
		FMenuBuilder& MenuBuilder,
		const FMenuParams& MenuParams,
		const FObjectActionParams& ObjectActionParams
	);
	
protected:
	// Helper struct to track the util to call a function on
	struct FFunctionAndUtil
	{
		FFunctionAndUtil(const FBlutilityFunctionData& InFunctionData, const TSharedRef<FAssetActionUtilityPrototype>& InUtil, TSet<int32>& InSelection)
			: FunctionData(InFunctionData)
			, Util(InUtil) 
			, SelectionIndices(InSelection)
		{}

		bool operator==(const FFunctionAndUtil& InFunction) const
		{
			return FunctionData == InFunction.FunctionData && Util == InFunction.Util;
		}

		UFunction* GetFunction() const
		{
			if (const UObject* UtilityCDO = Util->LoadUtilityAsset())
			{
				if (const UClass* UtilityClass = UtilityCDO->GetClass())
				{
					return UtilityClass->FindFunctionByName(FunctionData.Name);
				}
			}

			return nullptr;
		}

		FBlutilityFunctionData FunctionData;
		TSharedRef<FAssetActionUtilityPrototype> Util;
		/** Indices to original object selection array which are supported by this utility */
		TSet<int32> SelectionIndices;
	};

protected:
	template<typename SelectionType>
	static void CreateBlutilityActionsMenu(FMenuBuilder& MenuBuilder, TMap<TSharedRef<FAssetActionUtilityPrototype>, TSet<int32>> Utils, const FText& MenuLabel, const FText& MenuToolTip, TFunction<bool(const FProperty* Property)> IsValidPropertyType, const TArray<SelectionType> Selection, const FName& IconName = "GraphEditor.Event_16x");
	template<typename SelectionType>
	static void CreateBlutilityActionsMenu(FToolMenuSection& InSection, TMap<TSharedRef<FAssetActionUtilityPrototype>, TSet<int32>> Utils, const FName& MenuName, const FText& MenuLabel, const FText& MenuToolTip, TFunction<bool(const FProperty* Property)> IsValidPropertyType, const TArray<SelectionType> Selection, const FName& IconName = "GraphEditor.Event_16x");
	template<typename SelectionType>
	static void CreateBlulityActionsSection(FMenuBuilder& InMenuBuilder, const FObjectActionParams& InObjectActionParams);
	static void OpenEditorForUtility(const FFunctionAndUtil& FunctionAndUtil);
	static void ExtractFunctions(const TMap<TSharedRef<FAssetActionUtilityPrototype>, TSet<int32>>& Utils, TMap<FString, TArray<FFunctionAndUtil>>& OutCategoryFunctions);
};

#undef UE_API