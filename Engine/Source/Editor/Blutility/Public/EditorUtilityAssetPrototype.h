// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetActionUtility.h"
#include "UObject/Object.h"
#include "AssetRegistry/AssetData.h"
#include "IEditorUtilityExtension.h"

#include "EditorUtilityAssetPrototype.generated.h"

USTRUCT()
struct FBlutilityFunctionData
{
	GENERATED_BODY()

public:
	UPROPERTY()
	TSoftClassPtr<UClass> Class;
	
	UPROPERTY()
	FName Name;
	
	UPROPERTY()
	FText NameText;

	UPROPERTY()
	FString Category;

	UPROPERTY()
	FText TooltipText;

	bool operator !=(const FBlutilityFunctionData& Other) const
	{
		return !(*this == Other);
	}

	bool operator == (const FBlutilityFunctionData& Other) const
	{
		return Class == Other.Class && Name == Other.Name;
	}
};

namespace AssetActionUtilityTags
{
	extern const FName BlutilityTagVersion;
	extern const FName SupportedClasses;
	extern const FName IsActionForBlueprint;
	extern const FName CallableFunctions;
	
	extern const int32 TagVersion;
}

#define UE_API BLUTILITY_API

class FAssetActionUtilityPrototype
{
public:
	FAssetActionUtilityPrototype(const FAssetData& SourceAsset)
		: UtilityBlueprintAsset(SourceAsset)
	{
	}

	UObject* LoadUtilityAsset() const;

	UE_API bool IsLatestVersion() const;
	bool AreSupportedClassesForBlueprints() const;
	UE_API TArray<TSoftClassPtr<UObject>> GetSupportedClasses() const;
	TArray<FAssetActionSupportCondition> GetAssetActionSupportConditions() const;
	UE_API TArray<FBlutilityFunctionData> GetCallableFunctions() const;

	const FAssetData& GetUtilityBlueprintAsset() const { return UtilityBlueprintAsset; }

	bool operator !=(const FAssetActionUtilityPrototype& Other) const
	{
		return !(*this == Other);
	}

	bool operator == (const FAssetActionUtilityPrototype& Other) const
	{
		return UtilityBlueprintAsset.PackageName == Other.UtilityBlueprintAsset.PackageName;
	}

public:
	static void AddTagsFor_Version(FAssetRegistryTagsContext Context);
	static void AddTagsFor_SupportedClasses(const TArray<TSoftClassPtr<UObject>>& SupportedClasses, FAssetRegistryTagsContext Context);
	static void AddTagsFor_SupportedConditions(const TArray<FAssetActionSupportCondition>& SupportedConditions, FAssetRegistryTagsContext Context);
	static void AddTagsFor_IsActionForBlueprints(bool IsActionForBlueprints, FAssetRegistryTagsContext Context);
	static void AddTagsFor_CallableFunctions(const UObject* FunctionsSource, FAssetRegistryTagsContext Context);

private:
	FAssetData UtilityBlueprintAsset;
};

FORCEINLINE uint32 GetTypeHash(const FAssetActionUtilityPrototype& ExtensionAsset)
{
	return HashCombine(GetTypeHash(ExtensionAsset.GetUtilityBlueprintAsset().PackageName), GetTypeHash(ExtensionAsset.GetUtilityBlueprintAsset().AssetName));
}

#undef UE_API