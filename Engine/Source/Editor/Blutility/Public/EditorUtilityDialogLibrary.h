// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "GenericPlatform/GenericPlatformMisc.h"
#include "EditorUtilityDialogResult.h"
#include "EditorUtilityDialogWidget.h"

#include "EditorUtilityDialogLibrary.generated.h"

class SEditorUtilityDialog;

#define UE_API BLUTILITY_API

/**
 * Blueprint Function Library for creating modal and non-modal dialogs
 * that contain Editor Utility Widgets.
 */
UCLASS(MinimalAPI)
class UEditorUtilityDialogLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** Creates an Editor Utility Dialog Widget and shows it as a modal window. Blocks until dismissed. */
	UFUNCTION(BlueprintCallable, Category = "Editor Utility|Dialog")
	static UE_API FEditorUtilityDialogResult ShowModalEditorUtilityDialog(TSubclassOf<UEditorUtilityDialogWidget> WidgetClass, FText Title, EAppMsgType::Type ButtonLayout = EAppMsgType::OkCancel, FVector2D Size = FVector2D::ZeroVector);

	/** Creates an Editor Utility Dialog Widget as a non-modal window. Returns immediately. OnClosed fires when dismissed. */
	UFUNCTION(BlueprintCallable, Category = "Editor Utility|Dialog", meta = (AutoCreateRefTerm = "OnClosed", DeterminesOutputType = "WidgetClass"))
	static UE_API UEditorUtilityDialogWidget* ShowEditorUtilityDialog(TSubclassOf<UEditorUtilityDialogWidget> WidgetClass, const FOnEditorUtilityDialogClosed& OnClosed, FText Title = FText::GetEmpty(), EAppMsgType::Type ButtonLayout = EAppMsgType::OkCancel, FVector2D Size = FVector2D::ZeroVector);

	/** Creates and initializes a dialog widget without showing it. Set data on it, then pass to Show Modal/Non-Modal (Instance). */
	UFUNCTION(BlueprintCallable, Category = "Editor Utility|Dialog", meta = (DeterminesOutputType = "WidgetClass"))
	static UE_API UEditorUtilityDialogWidget* CreateEditorUtilityDialogWidget(TSubclassOf<UEditorUtilityDialogWidget> WidgetClass);

	/** Shows a pre-constructed Editor Utility Dialog Widget as a modal window. */
	UFUNCTION(BlueprintCallable, Category = "Editor Utility|Dialog")
	static UE_API FEditorUtilityDialogResult ShowModalEditorUtilityDialogInstance(UEditorUtilityDialogWidget* WidgetInstance, FText Title, EAppMsgType::Type ButtonLayout = EAppMsgType::OkCancel, FVector2D Size = FVector2D::ZeroVector);

	/** Shows a pre-constructed Editor Utility Dialog Widget as a non-modal window. */
	UFUNCTION(BlueprintCallable, Category = "Editor Utility|Dialog", meta = (AutoCreateRefTerm = "OnClosed"))
	static UE_API void ShowEditorUtilityDialogInstance(UEditorUtilityDialogWidget* WidgetInstance, const FOnEditorUtilityDialogClosed& OnClosed, FText Title = FText::GetEmpty(), EAppMsgType::Type ButtonLayout = EAppMsgType::OkCancel, FVector2D Size = FVector2D::ZeroVector);

private:
	static UE_API TPair<UEditorUtilityDialogWidget*, TSharedPtr<SEditorUtilityDialog>> CreateDialogInternal(TSubclassOf<UEditorUtilityDialogWidget> WidgetClass, const FText& Title, EAppMsgType::Type ButtonLayout, const FVector2D& Size);

	static UE_API TSharedPtr<SEditorUtilityDialog> WrapWidgetInDialog(UEditorUtilityDialogWidget* Widget, const FText& Title, EAppMsgType::Type ButtonLayout, const FVector2D& Size);

	/** Show the dialog non-modally and route the close callback through a holder that tracks rebuilds. */
	static UE_API void BindNonModalCloseCallback(SEditorUtilityDialog& Dialog, UEditorUtilityDialogWidget* InitialWidget, const FOnEditorUtilityDialogClosed& OnClosed);
};

#undef UE_API
