// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GenericPlatform/GenericPlatformMisc.h"
#include "UObject/Object.h"

#include "EditorUtilityDialogResult.generated.h"

class UEditorUtilityDialogWidget;

/** Delegate fired when an async (non-modal) Editor Utility Dialog is closed. */
DECLARE_DYNAMIC_DELEGATE_TwoParams(
	FOnEditorUtilityDialogClosed,
	EAppReturnType::Type, Result,
	UEditorUtilityDialogWidget*, WidgetInstance);

/**
 * Result returned by ShowModalEditorUtilityDialog.
 * Contains the button the user pressed and a reference to the widget instance
 * so the caller can read back any data the user entered.
 */
USTRUCT(MinimalAPI, BlueprintType)
struct FEditorUtilityDialogResult
{
	GENERATED_BODY()

	/** Which button the user pressed to close the dialog (Ok, Cancel, Yes, No, etc.). */
	UPROPERTY(BlueprintReadOnly, Category = "Dialog Result")
	TEnumAsByte<EAppReturnType::Type> ReturnType = EAppReturnType::Cancel;

	/** The widget instance shown in the dialog. Cast to your subclass to read back user-entered data. */
	UPROPERTY(BlueprintReadOnly, Category = "Dialog Result")
	TObjectPtr<UEditorUtilityDialogWidget> WidgetInstance = nullptr;
};
