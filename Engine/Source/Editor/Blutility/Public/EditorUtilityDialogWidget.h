// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorUtilityWidget.h"
#include "GenericPlatform/GenericPlatformMisc.h"

#include "EditorUtilityDialogWidget.generated.h"

class SEditorUtilityDialog;

#define UE_API BLUTILITY_API

/**
 * Base class for Editor Utility Widgets that are designed to be shown in modal or
 * non-modal dialog windows. Subclass this in Blueprint to create custom dialog content.
 *
 * Unlike regular Editor Utility Widgets (which live in dockable tabs), dialog widgets
 * are shown in standalone windows with a standard button bar (OK, Cancel, Yes/No, etc.).
 *
 * The caller receives this widget instance after the dialog closes, so you can expose
 * UPROPERTY fields that the caller reads back to get user-entered data.
 */
UCLASS(MinimalAPI, Abstract, Blueprintable, meta = (ShowWorldContextPin))
class UEditorUtilityDialogWidget : public UEditorUtilityWidget
{
	GENERATED_BODY()

public:
	/** The title shown in the modal window's title bar. Can be overridden by the caller when showing the dialog. */
	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Category = "Dialog Settings")
	FText DialogTitle;

	/** Default window size in pixels (Width, Height). Set to (0, 0) to auto-size to content. */
	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Category = "Dialog Settings")
	FVector2D DesiredDialogSize = FVector2D::ZeroVector;

	/** Which standard buttons appear at the bottom of the dialog (Ok, OkCancel, YesNo, etc.). */
	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Category = "Dialog Settings")
	TEnumAsByte<EAppMsgType::Type> ButtonLayout = EAppMsgType::OkCancel;

	/** Called after the widget is created, before the dialog becomes visible. Initialize state here. */
	UFUNCTION(BlueprintImplementableEvent, Category = "Dialog|Events")
	void OnDialogOpened();

	/** Called when the user presses a positive button (OK, Yes, etc.). Fires before close. */
	UFUNCTION(BlueprintImplementableEvent, Category = "Dialog|Events")
	void OnDialogConfirmed();

	/** Called when the user presses Cancel, No, or closes via the X button. Fires before close. */
	UFUNCTION(BlueprintImplementableEvent, Category = "Dialog|Events")
	void OnDialogCancelled();

	/** Called for ANY button press. Catch-all alternative to OnDialogConfirmed/OnDialogCancelled. */
	UFUNCTION(BlueprintImplementableEvent, Category = "Dialog|Events")
	void OnDialogButtonPressed(EAppReturnType::Type Result);

	/** Programmatically close this dialog with a specific result. */
	UFUNCTION(BlueprintCallable, Category = "Dialog")
	UE_API void CloseDialog(EAppReturnType::Type Result);

	/** Set by the dialog system when the widget is hosted in an SEditorUtilityDialog. */
	void SetOwningDialog(TSharedPtr<SEditorUtilityDialog> InDialog);

private:
	/** Weak reference to the Slate dialog window hosting this widget. Used by CloseDialog(). */
	TWeakPtr<SEditorUtilityDialog> OwningDialog;
};

#undef UE_API
