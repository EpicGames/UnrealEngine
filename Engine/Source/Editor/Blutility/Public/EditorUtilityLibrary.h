// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Kismet/BlueprintAsyncActionBase.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Math/UnrealMathSSE.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/UObjectGlobals.h"

#include "EditorUtilityLibrary.generated.h"

#define UE_API BLUTILITY_API

class UWidget;
class UEditorUtilityWidgetBlueprint;
class AActor;
class UClass;
class UEditorPerProjectUserSettings;
class UWorld;
struct FAssetData;
struct FContentBrowserItemPath;
struct FFrame;

UENUM(BlueprintType)
enum class ECastToWidgetBlueprintCases : uint8
{
	CastSucceeded,
	CastFailed
};

UCLASS(MinimalAPI)
class UEditorUtilityBlueprintAsyncActionBase : public UBlueprintAsyncActionBase
{
	GENERATED_UCLASS_BODY()

public:
	UE_API virtual void RegisterWithGameInstance(const UObject* WorldContextObject) override;
	UE_API virtual void SetReadyToDestroy() override;
};

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FAsyncDelayComplete);

UCLASS(MinimalAPI)
class UAsyncEditorDelay : public UEditorUtilityBlueprintAsyncActionBase
{
	GENERATED_UCLASS_BODY()

public:
#if WITH_EDITOR

	UFUNCTION(BlueprintCallable, meta = (BlueprintInternalUseOnly = "true"))
	static UE_API UAsyncEditorDelay* AsyncEditorDelay(float Seconds, int32 MinimumFrames = 30);

#endif

public:

	UPROPERTY(BlueprintAssignable)
	FAsyncDelayComplete Complete;

public:

	UE_API void Start(float InMinimumSeconds, int32 InMinimumFrames);

private:

	UE_API bool HandleComplete(float DeltaTime);

	uint64 EndFrame = 0;
	double EndTime = 0;
};

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FAsyncEditorWaitForGameWorldEvent, UWorld*, World);

UCLASS(MinimalAPI)
class UAsyncEditorWaitForGameWorld : public UEditorUtilityBlueprintAsyncActionBase
{
	GENERATED_UCLASS_BODY()

public:
#if WITH_EDITOR

	UFUNCTION(BlueprintCallable, meta = (BlueprintInternalUseOnly = "true"))
	static UE_API UAsyncEditorWaitForGameWorld* AsyncWaitForGameWorld(int32 Index = 0, bool Server = false);

#endif

public:

	UPROPERTY(BlueprintAssignable)
	FAsyncEditorWaitForGameWorldEvent Complete;

public:

	UE_API void Start(int32 Index, bool Server);

private:

	UE_API bool OnTick(float DeltaTime);

	int32 Index;
	bool Server;
};

UCLASS(MinimalAPI)
class UAsyncEditorOpenMapAndFocusActor : public UEditorUtilityBlueprintAsyncActionBase
{
	GENERATED_UCLASS_BODY()

public:
#if WITH_EDITOR

	UFUNCTION(BlueprintCallable, meta = (BlueprintInternalUseOnly = "true"))
	static UE_API UAsyncEditorOpenMapAndFocusActor* AsyncEditorOpenMapAndFocusActor(FSoftObjectPath Map, FString FocusActorName);

#endif

public:

	UPROPERTY(BlueprintAssignable)
	FAsyncDelayComplete Complete;

public:

	UE_API void Start(FSoftObjectPath InMap, FString InFocusActorName);

private:

	UE_API bool OnTick(float DeltaTime);

	FSoftObjectPath Map;
	FString FocusActorName;
};


// Expose editor utility functions to Blutilities 
UCLASS(MinimalAPI)
class UEditorUtilityLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()

public:
#if WITH_EDITOR

	UFUNCTION(BlueprintCallable, Category = "Development|Editor")
	static UE_API TArray<AActor*> GetSelectionSet();

	UFUNCTION(BlueprintCallable, Category = "Development|Editor")
	static UE_API void GetSelectionBounds(FVector& Origin, FVector& BoxExtent, float& SphereRadius);

	// Gets the set of currently selected assets
	UFUNCTION(BlueprintCallable, Category = "Development|Editor")
	static UE_API TArray<UObject*> GetSelectedAssets();
	
	UFUNCTION(BlueprintCallable, Category = "Development|Editor")
	static UE_API TArray<UObject*> GetSelectedAssetsOfClass(UClass* AssetClass);

	// Gets the set of currently selected classes
	UFUNCTION(BlueprintCallable, Category = "Development|Editor")
	static UE_API TArray<UClass*> GetSelectedBlueprintClasses();

	// Gets the set of currently selected asset data
	UFUNCTION(BlueprintCallable, Category = "Development|Editor")
	static UE_API TArray<FAssetData> GetSelectedAssetData();

	// Renames an asset (cannot move folders)
	UFUNCTION(BlueprintCallable, Category = "Development|Editor")
	static UE_API void RenameAsset(UObject* Asset, const FString& NewName);

	/**
	* Attempts to find the actor specified by PathToActor in the current editor world
	* @param	PathToActor	The path to the actor (e.g. PersistentLevel.PlayerStart)
	* @return	A reference to the actor, or none if it wasn't found
	*/
	UFUNCTION(BlueprintPure, Category = "Development|Editor")
	UE_API AActor* GetActorReference(FString PathToActor);

	/**
	 * Attempts to get the path for the active content browser, returns false if there is no active content browser
	 * or if it was a virtual path
	 * @param	OutPath	The returned path if successfully found
	 * @return	Whether a path was successfully returned
	 */
	UFUNCTION(BlueprintPure, Category = "Development|Editor")
	static UE_API bool GetCurrentContentBrowserPath(FString& OutPath);

	// Gets the current content browser path if one is open, whether it is internal or virtual.
	UFUNCTION(BlueprintPure, Category = "Development|Editor")
	static UE_API FContentBrowserItemPath GetCurrentContentBrowserItemPath();

	// Gets the path to the currently selected folder in the content browser
	UFUNCTION(BlueprintPure, Category = "Development|Editor")
	static UE_API TArray<FString> GetSelectedFolderPaths();
	
	// Returns the folders that are selected in the path view for the content browser
    UFUNCTION(BlueprintPure, Category = "Development|Editor")
    static UE_API TArray<FString> GetSelectedPathViewFolderPaths();

	/**
	 * Sync the Content Browser to the given folder(s)
	 * @param	FolderList	The list of folders to sync to in the Content Browser
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Content Browser")
	static UE_API void SyncBrowserToFolders(const TArray<FString>& FolderList);
	
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Content Browser")
	static UE_API void ConvertToEditorUtilityWidget(class UWidgetBlueprint* WidgetBP);
	
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Widget Blueprint", Meta = (ExpandEnumAsExecs = "Branches"))
	static UE_API void CastToWidgetBlueprint(UObject* Object, ECastToWidgetBlueprintCases& Branches, UWidgetBlueprint*& AsWidgetBlueprint);

	/** Searches the blueprint's widget hierarchy for a widget with the specified name */
	UFUNCTION(BlueprintCallable, BlueprintPure = false, Category = "Editor Scripting | Widget Blueprint")
	static UE_API UWidget* FindSourceWidgetByName(UWidgetBlueprint* WidgetBlueprint, FName WidgetName);

	/**
	 * Create a new widget and add to the specific widget blueprint's widget tree
	 *
	 * @param WidgetBlueprint The widget blueprint to add a widget to
	 * @param WidgetClass The widget class to add to the widget blueprint
	 * @param WidgetName The name to give the new widget
	 * @param WidgetParentName The name of the existing widget that will hold the new widget. Must be an existing Panel Widget or none of if the widget tree is empty and the new widget will become the RootWidget.
	 *
	 * @return The widget that was created and added to the widget blueprint
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure = false, Category = "Editor Scripting | Widget Blueprint")
	static UE_API UWidget* AddSourceWidget(UWidgetBlueprint* WidgetBlueprint, TSubclassOf<UWidget> WidgetClass, FName WidgetName, FName WidgetParentName);
#endif
};

#undef UE_API
