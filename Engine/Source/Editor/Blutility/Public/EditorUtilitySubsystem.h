// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Queue.h"
#include "Containers/Set.h"
#include "Containers/Ticker.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "EditorSubsystem.h"
#include "Templates/SharedPointer.h"
#include "Templates/SubclassOf.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UnrealEdMisc.h"

#include "EditorUtilitySubsystem.generated.h"

#define UE_API BLUTILITY_API

class FMenuBuilder;
class FOutputDevice;
class FSpawnTabArgs;
class FSubsystemCollectionBase;
class IConsoleObject;
class SDockTab;
class SWindow;
class UClass;
class UBlueprintGeneratedClass;
class UEditorUtilityTask;
class UEditorUtilityWidget;
class UEditorUtilityWidgetBlueprint;
class UWidgetBlueprintGeneratedClass;
class UWorld;
struct FFrame;
struct FSoftObjectPath;

/** Delegate for a PIE event exposed via Editor Utility (begin, end, pause/resume, etc) */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnEditorUtilityPIEEvent, const bool, bIsSimulating);

UCLASS(MinimalAPI, config = EditorPerProjectUserSettings)
class UEditorUtilitySubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:
	UE_API UEditorUtilitySubsystem();

	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection);
	UE_API virtual void Deinitialize();

	static UE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

	UE_API void MainFrameCreationFinished(TSharedPtr<SWindow> InRootWindow, bool bIsRunningStartupDialog);
	UE_API void HandleStartup();

	UPROPERTY(config)
	TArray<FSoftObjectPath> LoadedUIs;

	UPROPERTY(config)
	TArray<FSoftObjectPath> StartupObjects;

	TMap<FName, UEditorUtilityWidgetBlueprint*> RegisteredTabs;
	TMap<FName, UWidgetBlueprintGeneratedClass*> RegisteredTabsByGeneratedClass;

	// Allow startup object to be garbage collected
	UFUNCTION(BlueprintCallable, Category = "Development|Editor")
	UE_API void ReleaseInstanceOfAsset(UObject* Asset);

	UFUNCTION(BlueprintCallable, Category = "Development|Editor")
	UE_API bool TryRun(UObject* Asset);

	UFUNCTION(BlueprintCallable, Category = "Development|Editor")
	UE_API bool TryRunClass(UClass* ObjectClass);

	UFUNCTION(BlueprintCallable, Category = "Development|Editor")
	UE_API bool CanRun(UObject* Asset) const;

	UFUNCTION(BlueprintCallable, Category = "Development|Editor")
	UE_API UEditorUtilityWidget* SpawnAndRegisterTabAndGetID(class UEditorUtilityWidgetBlueprint* InBlueprint, FName& NewTabID);

	UFUNCTION(BlueprintCallable, Category = "Development|Editor")
	UE_API UEditorUtilityWidget* SpawnAndRegisterTab(class UEditorUtilityWidgetBlueprint* InBlueprint);
	
	UFUNCTION(BlueprintCallable, Category = "Development|Editor")
	UE_API void RegisterTabAndGetID(class UEditorUtilityWidgetBlueprint* InBlueprint, FName& NewTabID);

	/**
	 * Unlike SpawnAndRegisterTabAndGetID allows spawn tab while providing TabID from Python scripts or BP
	 */
	UFUNCTION(BlueprintCallable, Category = "Development|Editor")
	UE_API UEditorUtilityWidget* SpawnAndRegisterTabWithId(class UEditorUtilityWidgetBlueprint* InBlueprint, FName InTabID);

	UFUNCTION(BlueprintCallable, Category = "Development|Editor")
	UE_API UEditorUtilityWidget* SpawnAndRegisterTabAndGetIDGeneratedClass(UWidgetBlueprintGeneratedClass* InGeneratedWidgetBlueprint, FName& NewTabID);

	UFUNCTION(BlueprintCallable, Category = "Development|Editor")
	UE_API UEditorUtilityWidget* SpawnAndRegisterTabGeneratedClass(UWidgetBlueprintGeneratedClass* InGeneratedWidgetBlueprint);

	UFUNCTION(BlueprintCallable, Category = "Development|Editor")
	UE_API void RegisterTabAndGetIDGeneratedClass(UWidgetBlueprintGeneratedClass* InGeneratedWidgetBlueprint, FName& NewTabID);

	/**
	 * Unlike SpawnAndRegisterTabAndGetID allows spawn tab while providing TabID from Python scripts or BP
	 */
	UFUNCTION(BlueprintCallable, Category = "Development|Editor")
	UE_API UEditorUtilityWidget* SpawnAndRegisterTabWithIdGeneratedClass(UWidgetBlueprintGeneratedClass* InGeneratedWidgetBlueprint, FName InTabID);

	/** Given an ID for a tab, try to find a tab spawner that matches, and then spawn a tab. Returns true if it was able to find a matching tab spawner */
	UFUNCTION(BlueprintCallable, Category = "Development|Editor")
	UE_API bool SpawnRegisteredTabByID(FName NewTabID);

	/** Given an ID for a tab, try to find an existing tab. Returns true if it found a tab. */
	UFUNCTION(BlueprintCallable, Category = "Development|Editor")
	UE_API bool DoesTabExist(FName NewTabID);

	/** Given an ID for a tab, try to find and close an existing tab. Returns true if it found a tab to close. */
	UFUNCTION(BlueprintCallable, Category = "Development|Editor")
	UE_API bool CloseTabByID(FName NewTabID);

	/** Given an ID for a tab, try to close and unregister a tab that was registered through this subsystem */
	UFUNCTION(BlueprintCallable, Category = "Development|Editor")
	UE_API bool UnregisterTabByID(FName TabID);

	/** Given an editor utility widget blueprint, get the widget it creates. This will return a null pointer if the widget is not currently in a tab.*/
	UFUNCTION(BlueprintCallable, Category = "Development|Editor")
	UE_API UEditorUtilityWidget* FindUtilityWidgetFromBlueprint(class UEditorUtilityWidgetBlueprint* InBlueprint);

	/** Given an editor utility widget blueprint, get the TabID for it. This will return None if the widget is not currently in a tab.*/
	UFUNCTION(BlueprintCallable, Category = "Development|Editor")
	UE_API FName GetTabIDFromBlueprint(class UEditorUtilityWidgetBlueprint* InBlueprint);

	/**  */
	UFUNCTION(BlueprintCallable, Category = "Development|Editor")
	UE_API void RegisterAndExecuteTask(UEditorUtilityTask* NewTask, UEditorUtilityTask* OptionalParentTask = nullptr);

	UE_API void RemoveTaskFromActiveList(UEditorUtilityTask* Task);

	UE_API void RegisterReferencedObject(UObject* ObjectToReference);
	UE_API void UnregisterReferencedObject(UObject* ObjectToReference);

	/** Expose Begin PIE to blueprints.*/
	UPROPERTY(BlueprintAssignable)
	FOnEditorUtilityPIEEvent OnBeginPIE;

	/** Expose End PIE to blueprints.*/
	UPROPERTY(BlueprintAssignable)
	FOnEditorUtilityPIEEvent OnEndPIE;

protected:
	UE_API void HandleStartupAssets();
	UE_API void HandleDynamicStartupAssets(TConstArrayView<FAssetData> InAssets);

	UE_API void OnBlueprintGeneratedClassUnloaded(UBlueprintGeneratedClass* BPGC);

	UEditorUtilityTask* GetActiveTask() { return ActiveTaskStack.Num() > 0 ? ActiveTaskStack[ActiveTaskStack.Num() - 1] : nullptr; };

	UE_API void StartTask(UEditorUtilityTask* Task);

	UE_API bool Tick(float DeltaTime);

	UE_API void ProcessRunTaskCommands();

	UE_API void RunTaskCommand(const TArray<FString>& Params, UWorld* InWorld, FOutputDevice& Ar);
	UE_API void CancelAllTasksCommand(const TArray<FString>& Params, UWorld* InWorld, FOutputDevice& Ar);

	UE_API UClass* FindClassByName(const FString& RawTargetName);
	UE_API UClass* FindBlueprintClass(const FString& TargetNameRaw);

	/** Called when Play in Editor begins. */
	UE_API void HandleOnBeginPIE(const bool bIsSimulating);

	/** Called when Play in Editor stops. */
	UE_API void HandleOnEndPIE(const bool bIsSimulating);

	UE_API TSharedRef<SDockTab> SpawnEditorUITabFromGeneratedClass(const FSpawnTabArgs& SpawnTabArgs, UWidgetBlueprintGeneratedClass* InGeneratedWidgetBlueprint);

	UE_API void OnSpawnedFromGeneratedClassTabClosed(TSharedRef<SDockTab> TabBeingClosed);

	UE_API void OnMapChanged(UWorld* World, EMapChangeType MapChangeType);

	TMap<TSharedRef<SDockTab>, UEditorUtilityWidget*> SpawnedFromGeneratedClassTabs;

private:
	FName GenerateNewTabID(const UObject* WidgetObject, const FName& OptionalPostfix);
	
	void ExtendTabContextMenu(FMenuBuilder& InMenuBuilder, FName InTabId);
	void OnBrowseAssetInContentBrowser(FName InTabId);
	bool OnCanBrowseAssetInContentBrowser(FName InTabId);

	IConsoleObject* RunTaskCommandObject = nullptr;
	IConsoleObject* CancelAllTasksCommandObject = nullptr;
	
	UPROPERTY()
	TMap<TObjectPtr<UObject> /*Asset*/, TObjectPtr<UObject> /*Instance*/> ObjectInstances;

	TQueue< TArray<FString> > RunTaskCommandBuffer;

	/** AddReferencedObjects is used to report these references to GC. */
	TMap<TObjectPtr<UEditorUtilityTask>, TArray<TObjectPtr<UEditorUtilityTask>>> PendingTasks;

	UPROPERTY(Transient)
	TArray<TObjectPtr<UEditorUtilityTask>> ActiveTaskStack;

	FTSTicker::FDelegateHandle TickerHandle;

	/** List of objects that are being kept alive by this subsystem. */
	UPROPERTY()
	TSet<TObjectPtr<UObject>> ReferencedObjects;
};

#undef UE_API
