// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorConfigBase.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/SoftObjectPath.h"
#include "Engine/EngineTypes.h"
#include "Misc/NamePermissionList.h"
#include "WidgetEditingProjectSettings.h"
#include "EditorUtilityWidgetProjectSettings.generated.h"

#define UE_API BLUTILITY_API

class UWidgetCompilerRule;
class UUserWidget;
class UWidgetBlueprint;
class UPanelWidget;

/**
 * Implements the settings for Editor Utility Widget Editing Project Settings
 */
UCLASS(MinimalAPI, config=Editor, defaultconfig)
class UEditorUtilityWidgetProjectSettings : public UWidgetEditingProjectSettings
{
	GENERATED_BODY()

public:

	UE_API virtual FText GetSectionText() const override;
	UE_API virtual FText GetSectionDescription() const override;

	// Begin UObject Interface
	UE_API virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent) override;
	// End UObject Interface

	/** Get the permission list that controls actor actions */
	UE_API FNamePermissionList& GetAllowedEditorUtilityActorActions();
	UE_API const FNamePermissionList& GetAllowedEditorUtilityActorActions() const;

	/** Get the permission list that controls asset actions */
	UE_API FNamePermissionList& GetAllowedEditorUtilityAssetActions();
	UE_API const FNamePermissionList& GetAllowedEditorUtilityAssetActions() const;

	/** Set true to search generated classes for scripted actions. */
	UPROPERTY(EditAnywhere, config, Category = Designer)
	bool bSearchGeneratedClassesForScriptedActions = false;

protected:

	/** EditorUtility actor actions to always consider during context menu creation */
	FNamePermissionList AllowedEditorUtilityActorActions;

	/** EditorUtility asset actions to always consider during context menu creation */
	FNamePermissionList AllowedEditorUtilityAssetActions;
};

#undef UE_API
