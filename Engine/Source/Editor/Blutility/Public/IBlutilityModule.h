// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "AssetTypeCategories.h"

struct FAssetCategoryPath;
class UBlueprint;

/**
 * The public interface of BlutilityModule
 */
class IBlutilityModule : public IModuleInterface
{
public:

	/** Returns if the blueprint is an editor utility blueprint or widget */
	virtual bool IsEditorUtilityBlueprint( const UBlueprint* Blueprint ) const = 0;

	/** Global Find Results workspace menu item */
	virtual TSharedPtr<class FWorkspaceItem> GetMenuGroup() const = 0;

	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const = 0;

	virtual void AddLoadedScriptUI(class UEditorUtilityWidgetBlueprint* InBlueprint) = 0;

	virtual void RemoveLoadedScriptUI(class UEditorUtilityWidgetBlueprint* InBlueprint) = 0;
};

