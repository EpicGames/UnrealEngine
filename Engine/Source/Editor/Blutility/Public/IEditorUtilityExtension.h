// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"

#include "IEditorUtilityExtension.generated.h"

UINTERFACE(MinimalAPI, BlueprintType)
class UEditorUtilityExtension : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class IEditorUtilityExtension
{
	GENERATED_IINTERFACE_BODY()
};
