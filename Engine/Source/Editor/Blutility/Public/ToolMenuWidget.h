// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/Widget.h"
#include "Widgets/SNullWidget.h"

#include "ToolMenuWidget.generated.h"

enum class EMultiBoxType : uint8;

/**
 * A base ToolMenu widget that can support Menus and Toolbars.
 * 
 * Menus can be modified and added to with Blueprint or Python commands using the ToolMenus system.
 * 
 */
UCLASS(MinimalAPI)
class UToolMenuWidget : public UWidget
{
	GENERATED_BODY()

	UToolMenuWidget(const FObjectInitializer& ObjectInitializer);

public:
	/** Name of toolbar for registering with ToolMenus.
	*	This has the editor utility widget pre-pended to it to make the FullMenuName.
	*/
	UPROPERTY(Category = Config, EditAnywhere, BlueprintReadOnly)
	FString MenuName;

	UPROPERTY(Category = Config, EditAnywhere, BlueprintReadOnly)
	EMultiBoxType MenuType;

public:

#if WITH_EDITOR
	//~ Begin UObject Interface
	BLUTILITY_API virtual void PostInitProperties();
	//~ End UObject Interface

	//~ Begin UObject Interface
	BLUTILITY_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject Interface
#endif //WITH_EDITOR

protected:
	TSharedRef<SWidget> MyToolMenu = SNullWidget::NullWidget;
	
	/** The final usable name that can be retrieved by Python/Blueprint.
	*	Use this value when trying to Extend menus.
	*/
	UPROPERTY(Category = Config, BlueprintReadOnly, VisibleAnywhere)
	FName FullMenuName;

protected:
	//~ Begin UWidget Interface
	BLUTILITY_API virtual TSharedRef<SWidget> RebuildWidget() override;
	//~ End UWidget Interface
	
private:
	void UpdateFullMenuName();
};
